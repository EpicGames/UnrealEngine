// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
	public class MassEntityDebugger : ModuleRules
	{
		public MassEntityDebugger(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"InputCore",
					"MassCore",
					"MassEntity",
					"SlateCore",
					"Slate",
					"WorkspaceMenuStructure",
					"Projects",
					"UMG"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"ApplicationCore",
					"Json",
					"ToolWidgets"
				}
			);

			if (Target.bBuildEditor == true)
			{
				PublicDependencyModuleNames.AddRange(
					new string[] {
						"UnrealEd"
					}
				);
			}
		}
	}
}