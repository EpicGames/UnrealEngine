// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassDebuggerModel.h"
#include "MassProcessor.h"
#include "MassEntityManager.h"
#include "MassEntityQuery.h"
#include "MassProcessingPhaseManager.h"
#include "Engine/Engine.h"
#include "MassDebugger.h"
#include "MassDebuggerSettings.h"
#include "UObject/UObjectIterator.h"
#include "Containers/UnrealString.h"
#include "MassArchetypeData.h"
#include "Engine/World.h"
#include "SMassEntitiesView.h"
#include "SMassQueryEditorView.h"
#include "SMassDebugger.h"
#include "Algo/AnyOf.h"
#include "Misc/FileHelper.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonWriter.h"
#include "Dom/JsonObject.h"
#include "Misc/PackageName.h"
#include "UObject/CoreRedirects.h"

#define LOCTEXT_NAMESPACE "SMassDebugger"

namespace UE::Mass::Debugger::Private
{
	template<typename TBitSet>
	int32 BitSetDistance(const TBitSet& A, const TBitSet& B)
	{
		return (A - B).CountStoredTypes() + (B - A).CountStoredTypes();
	}

	float CalcArchetypeBitDistance(FMassDebuggerArchetypeData& A, FMassDebuggerArchetypeData& B)
	{
		int32 TotalLength = A.Composition.CountStoredTypes() +  B.Composition.CountStoredTypes();

		check(TotalLength > 0);

		return float(BitSetDistance(A.Composition.GetElementsBitSet(), B.Composition.GetElementsBitSet())) 
			/ TotalLength;
	}

	void MakeDisplayName(const FString& InName, FString& OutDisplayName)
	{
		OutDisplayName = InName;

		const UMassDebuggerSettings* DebuggerSettings = GetDefault<UMassDebuggerSettings>();
		check(DebuggerSettings);
		if (DebuggerSettings->bStripMassPrefix == false)
		{
			return;
		}

		OutDisplayName.RemoveFromStart(TEXT("Default__"), ESearchCase::CaseSensitive);
		OutDisplayName.RemoveFromStart(TEXT("Mass"), ESearchCase::CaseSensitive);
	}

	uint32 CalcProcessorHash(const UMassProcessor& Processor)
	{
		return PointerHash(&Processor);
	}

	/** We're ignoring all the CDO processors (since as such are not being run at runtime) as well ass processors owned
	 *  by a CDO, for the very same reason. */
	bool IsDebuggableProcessor(const UWorld* ContextWorld, const UMassProcessor& Processor)
	{
		return IsValid(&Processor)
			&& Processor.HasAnyFlags(RF_ClassDefaultObject) == false
			&& Processor.GetWorld() == ContextWorld 
			// checking ContextWorld is a cheaper way of supporting the declared behavior, since if there is a world then
			// the processors are definitely not CDO owned (by design). Is there is no world we need to check specifically.
			&& (ContextWorld != nullptr || Processor.GetOuter()->HasAnyFlags(RF_ClassDefaultObject) == false);
	}
} // namespace UE::Mass::Debugger::Private

//----------------------------------------------------------------------//
// FMassDebuggerEnvironment
//----------------------------------------------------------------------//
FString FMassDebuggerEnvironment::GetDisplayName() const
{
	FString DisplayName;

#if WITH_MASSENTITY_DEBUG
	if (TSharedPtr<const FMassEntityManager> EntityManagerPtr = GetEntityManager())
	{
		DisplayName += EntityManagerPtr->DebugGetName();
		if (DisplayName.Len())
		{
			DisplayName += TEXT(" - ");
		}
	}
#endif // WITH_MASSENTITY_DEBUG
	
	const UWorld* WorldPtr = World.Get();
	DisplayName += WorldPtr ? WorldPtr->GetDebugDisplayName() : TEXT("No World");
	return DisplayName;
}

TSharedPtr<const FMassEntityManager> FMassDebuggerEnvironment::GetEntityManager() const
{
	return EntityManager.Pin();
}

TSharedPtr<FMassEntityManager> FMassDebuggerEnvironment::GetMutableEntityManager() const
{
#if WITH_MASSENTITY_DEBUG
	if (TSharedPtr<const FMassEntityManager> SharedConstEntityManager = GetEntityManager())
	{
		FMassDebugger::FEnvironment* Environment = FMassDebugger::FindEnvironmentForEntityManager(*SharedConstEntityManager);
		if (Environment)
		{
			return Environment->MutableEntityManager.Pin();
		}
	}
#endif // WITH_MASSENTITY_DEBUG
	return nullptr;
}

//----------------------------------------------------------------------//
// FMassDebuggerQueryData
//----------------------------------------------------------------------//
FMassDebuggerQueryData::FMassDebuggerQueryData(const FMassEntityQuery& Query, const FText& InLabel)
	: Label(InLabel)
{
#if WITH_MASSENTITY_DEBUG
	FMassDebugger::GetQueryExecutionRequirements(Query, ExecutionRequirements);
	SourceQuery = Query;
#endif // WITH_MASSENTITY_DEBUG
}

FMassDebuggerQueryData::FMassDebuggerQueryData(const FMassSubsystemRequirements& SubsystemRequirements, const FText& InLabel)
	: Label(InLabel)
{
#if WITH_MASSENTITY_DEBUG
	SubsystemRequirements.ExportRequirements(ExecutionRequirements);
#endif // WITH_MASSENTITY_DEBUG
}

int32 FMassDebuggerQueryData::GetTotalBitsUsedCount() 
{
	return ExecutionRequirements.GetTotalBitsUsedCount();
}

bool FMassDebuggerQueryData::IsEmpty() const
{
	return ExecutionRequirements.IsEmpty();
}

//----------------------------------------------------------------------//
// FMassDebuggerProcessorData
//----------------------------------------------------------------------//
FMassDebuggerProcessorData::FMassDebuggerProcessorData(const UMassProcessor& InProcessor)
{
	SetProcessor(InProcessor);
#if WITH_MASSENTITY_DEBUG
	TConstArrayView<FMassEntityQuery*> ProcessorQueries = FMassDebugger::GetProcessorQueries(InProcessor);

	ProcessorRequirements = MakeShareable(new FMassDebuggerQueryData(InProcessor.GetProcessorRequirements(), LOCTEXT("MassProcessorRequirementsLabel", "Processor Requirements")));

	Queries.Reserve(ProcessorQueries.Num());
	for (const FMassEntityQuery* Query : ProcessorQueries)
	{
		check(Query);
		Queries.Add(MakeShareable(new FMassDebuggerQueryData(*Query, LOCTEXT("MassEntityQueryLabel", "Query"))));
	}
#endif // WITH_MASSENTITY_DEBUG
}

FMassDebuggerProcessorData::FMassDebuggerProcessorData(const FMassEntityManager& InEntityManager, const UMassProcessor& InProcessor
	, const TMap<FMassArchetypeHandle, TSharedPtr<FMassDebuggerArchetypeData>>& InTransientArchetypesMap)
{
	SetProcessor(InProcessor);
#if WITH_MASSENTITY_DEBUG
	EntityManager = InEntityManager.AsWeak();

	// yeah, it's ugly. But it's debugging code, so... 
	UMassProcessor& MutableProcessor = const_cast<UMassProcessor&>(InProcessor);
	TConstArrayView<FMassEntityQuery*> ProcessorQueries = FMassDebugger::GetUpToDateProcessorQueries(InEntityManager, MutableProcessor);

	ProcessorRequirements = MakeShareable(new FMassDebuggerQueryData(InProcessor.GetProcessorRequirements(), LOCTEXT("MassProcessorRequirementsLabel", "Processor Requirements")));

	const FMassEntityHandle SelectedEntityHandle = UE::Mass::Debug::bTestSelectedEntityAgainstProcessorQueries
		? FMassDebugger::GetSelectedEntity(InEntityManager)
		: FMassEntityHandle();
	FStringOutputDevice SelectedEntityFailureJustificationLog;
	SelectedEntityFailureJustificationLog.SetAutoEmitLineTerminator(true);
	const FText SelectedEntityHandleDescription = UE::Mass::Debug::bTestSelectedEntityAgainstProcessorQueries
		? FText::Format(LOCTEXT("WhyNotEntityJustificationLabel", "Why not entity {0}:"), FText::FromString(SelectedEntityHandle.DebugGetDescription()))
		: FText();

	Queries.Reserve(ProcessorQueries.Num());
	for (const FMassEntityQuery* Query : ProcessorQueries)
	{
		check(Query);
		TSharedPtr<FMassDebuggerQueryData>& QueryData = Queries.Add_GetRef(MakeShareable(new FMassDebuggerQueryData(*Query, LOCTEXT("MassEntityQueryLabel", "Query"))));

		if (SelectedEntityHandle.IsValid())
		{
			const FMassArchetypeHandle ArchetypeHandle = InEntityManager.GetArchetypeForEntity(SelectedEntityHandle);
			if (ArchetypeHandle.IsValid() && Query->GetArchetypes().Contains(ArchetypeHandle) == false)
			{
				if (FMassArchetypeHelper::DoesArchetypeMatchRequirements(FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ArchetypeHandle), *Query
					, false, &SelectedEntityFailureJustificationLog) == false)
				{
					FTextBuilder DescriptionBuilder;
					DescriptionBuilder.AppendLine(QueryData->AdditionalInformation);
					DescriptionBuilder.AppendLine(SelectedEntityHandleDescription);
					DescriptionBuilder.AppendLine(SelectedEntityFailureJustificationLog);
					QueryData->AdditionalInformation = DescriptionBuilder.ToText();
					
					SelectedEntityFailureJustificationLog.Reset();
				}
			}
		}

		for (const FMassArchetypeHandle& ArchetypeHandle : Query->GetArchetypes())
		{
			ValidArchetypes.Add(InTransientArchetypesMap.FindChecked(ArchetypeHandle));
		}
	}
#endif // WITH_MASSENTITY_DEBUG
}

void FMassDebuggerProcessorData::SetProcessor(const UMassProcessor& InProcessor)
{
	Name = InProcessor.GetProcessorName();
	UE::Mass::Debugger::Private::MakeDisplayName(Name, Label);

	Processor = &InProcessor;
	ProcessorHash = UE::Mass::Debugger::Private::CalcProcessorHash(InProcessor);
	bIsActive = InProcessor.IsActive();
	if (bIsActive == false)
	{
		Label.InsertAt(0, TEXT("[INACTIVE] "));
	}

#if WITH_MASSENTITY_DEBUG
	FStringOutputDevice DescriptionDevice;
	InProcessor.DebugOutputDescription(DescriptionDevice);
	if (DescriptionDevice != InProcessor.GetProcessorName())
	{
		Description = MoveTemp(DescriptionDevice);
	}
#endif // WITH_MASSENTITY_DEBUG
}

//----------------------------------------------------------------------//
// FMassDebuggerArchetypeData
//----------------------------------------------------------------------//
FMassDebuggerArchetypeData::FMassDebuggerArchetypeData(const FMassArchetypeHandle& ArchetypeHandle)
{
#if WITH_MASSENTITY_DEBUG
	Handle = ArchetypeHandle;
	Composition = FMassDebugger::GetArchetypeComposition(ArchetypeHandle);
	SparseElementsBitset = FMassDebugger::GetArchetypeSparseElementsBitset(ArchetypeHandle);

	// @todo should ensure we're using same hashing as the EntityManager here
	CompositionHash = Composition.CalculateHash();
	FullHash = CompositionHash;

	FString FullHashAsString;
	BytesToHexLower(reinterpret_cast<const uint8*>(&FullHash), sizeof(FullHash), FullHashAsString);
	HashLabel = FText::FromString(FullHashAsString);

	FMassDebugger::GetArchetypeEntityStats(ArchetypeHandle, ArchetypeStats);

	const TConstArrayView<FName> DebugNames = FMassDebugger::GetArchetypeDebugNames(ArchetypeHandle);

	if (DebugNames.IsEmpty())
	{
		// This archetype has no associated debug names, use hash as name.
		FString HashAsString; 
		BytesToHexLower(reinterpret_cast<const uint8*>(&CompositionHash), sizeof(CompositionHash), HashAsString);
		PrimaryDebugName = HashAsString;

		// Use first fragment as name
		if (FMassFragmentBitSet::FIndexIterator It = Composition.DebugGetFragments().GetIndexIterator())
		{
			const FName FirstStructName = Composition.DebugGetFragments().DebugGetStructTypeName(*It);
			TStringBuilder<256> StringBuilder;
			StringBuilder.Append(FirstStructName.ToString());
			StringBuilder.Append(TEXT("..."));
			Label = FText::FromString(StringBuilder.ToString());
		}
		else
		{
			Label = FText::FromString(HashAsString);
		}
		
		LabelLong = Label;
	}
	else
	{
		PrimaryDebugName = DebugNames[0].ToString();

		TStringBuilder<256> StringBuilder;

		// Short label for lists
		StringBuilder.Reset();
		StringBuilder.Append(DebugNames[0].ToString());
		if (DebugNames.Num() > 1)
		{
			StringBuilder.Append(TEXT("..."));
		}
		Label = FText::FromString(StringBuilder.ToString());

		// Longer label for info display
		StringBuilder.Reset();
		for (int i = 0; i < DebugNames.Num(); i++)
		{
			if (i > 0)
			{
				StringBuilder.Append(TEXT(", "));
			}
			StringBuilder.Append(DebugNames[i].ToString());
		}
		LabelLong = FText::FromString(StringBuilder.ToString());

		// Label tooltip
		StringBuilder.Reset();
		for (int i = 0; i < DebugNames.Num(); i++)
		{
			if (i > 0)
			{
				StringBuilder.Append(TEXT("\n"));
			}
			StringBuilder.Append(DebugNames[i].ToString());
		}
		LabelTooltip = FText::FromString(StringBuilder.ToString());
	}

#endif // WITH_MASSENTITY_DEBUG
}

int32 FMassDebuggerArchetypeData::GetTotalBitsUsedCount() const
{
	return Composition.CountStoredTypes();
}

//----------------------------------------------------------------------//
// FMassDebuggerFragmentData
//----------------------------------------------------------------------//
FMassDebuggerFragmentData::FMassDebuggerFragmentData(TNotNull<const UScriptStruct*> InFragment)
	: Fragment(InFragment)
{
	if (Fragment.IsValid())
	{
		Name = Fragment.Pin()->GetDisplayNameText();
	}
	else
	{
		Name = LOCTEXT("Invalid", "Invalid");
	}
}

//----------------------------------------------------------------------//
// FMassDebuggerProcessingGraphNode
//----------------------------------------------------------------------//
FMassDebuggerProcessingGraphNode::FMassDebuggerProcessingGraphNode(const TSharedPtr<FMassDebuggerProcessorData>& InProcessorData, const UMassCompositeProcessor::FDependencyNode& InProcessorNode)
	: ProcessorData(InProcessorData)
{
	if (InProcessorNode.Processor == nullptr)
	{
		return;
	}

	WaitForNodes = InProcessorNode.Dependencies;
}

FText FMassDebuggerProcessingGraphNode::GetLabel() const
{
	if (ProcessorData.IsValid())
	{
		return FText::FromString(ProcessorData->Label);
	}

	return LOCTEXT("Invalid", "Invalid");
}

//----------------------------------------------------------------------//
// FMassDebuggerProcessingGraph
//----------------------------------------------------------------------//
FMassDebuggerProcessingGraph::FMassDebuggerProcessingGraph(const FMassDebuggerModel& DebuggerModel, TNotNull<const UMassCompositeProcessor*> InGraphOwner)
{
	Label = InGraphOwner->GetProcessorName();
#if WITH_MASSENTITY_DEBUG
	TConstArrayView<UMassCompositeProcessor::FDependencyNode> ProcessingGraph = FMassDebugger::GetProcessingGraph(*InGraphOwner);

	if (ProcessingGraph.Num() > 0)
	{
		GraphNodes.Reserve(ProcessingGraph.Num());
		for (const UMassCompositeProcessor::FDependencyNode& Node : ProcessingGraph)
		{
			check(Node.Processor);
			const TSharedPtr<FMassDebuggerProcessorData>& ProcessorData = DebuggerModel.GetProcessorDataChecked(*Node.Processor);
			check(ProcessorData.IsValid());
			GraphNodes.Add(FMassDebuggerProcessingGraphNode(ProcessorData, Node));
		}
	}
	// it's possible for the graph to be empty if InGraphOwner has been populated for a single-thread execution.
	// See if there are any processors owned by InGraphOwner.
	else if (InGraphOwner->IsEmpty() == false)
	{
		TConstArrayView<TObjectPtr<UMassProcessor>> HostedProcessors = FMassDebugger::GetHostedProcessors(*InGraphOwner);
		for (const TObjectPtr<UMassProcessor>& Processor : HostedProcessors)
		{
			check(Processor);
			const TSharedPtr<FMassDebuggerProcessorData>& ProcessorData = DebuggerModel.GetProcessorDataChecked(*Processor);
			check(ProcessorData.IsValid());
			GraphNodes.Add(FMassDebuggerProcessingGraphNode(ProcessorData));
		}

		// if we have processors, but the flat processing graph is empty, it means it's a single-threaded composite processor
		bSingleTheadGraph = true;
	}
#endif // WITH_MASSENTITY_DEBUG
}

//----------------------------------------------------------------------//
// FMassDebuggerEnvironment
//----------------------------------------------------------------------//
FMassDebuggerEnvironment::FMassDebuggerEnvironment(const TSharedRef<const FMassEntityManager>& InEntityManager)
	: EntityManager(InEntityManager), World(InEntityManager->GetWorld()), bNeedsValidWorld(InEntityManager->GetWorld() != nullptr)
{
	
}

//----------------------------------------------------------------------//
// FMassDebuggerModel
//----------------------------------------------------------------------//
FMassDebuggerModel::FMassDebuggerModel()
{
#if WITH_MASSENTITY_DEBUG
	OnEntitySelectedHandle = FMassDebugger::OnEntitySelectedDelegate.AddRaw(this, &FMassDebuggerModel::OnEntitySelected);
	OnBreakpointsChangedHandle = FMassDebugger::OnBreakpointsChangedDelegate.AddRaw(this, &FMassDebuggerModel::OnBreakpointsChanged);
	LoadQueriesFromDisk();
	LoadBreakpointsFromDisk();
#endif // WITH_MASSENTITY_DEBUG
}

// disabling to avoid reporting CachedProcessors deprecation
PRAGMA_DISABLE_DEPRECATION_WARNINGS
FMassDebuggerModel::~FMassDebuggerModel()
{
#if WITH_MASSENTITY_DEBUG
	if (OnEntitySelectedHandle.IsValid())
	{
		FMassDebugger::OnEntitySelectedDelegate.Remove(OnEntitySelectedHandle);
	}
#endif // WITH_MASSENTITY_DEBUG
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FMassDebuggerModel::SetEnvironment(const TSharedPtr<FMassDebuggerEnvironment>& Item)
{
#if WITH_MASSENTITY_DEBUG
	if (Item)
	{
		Environment = Item;
		EnvironmentDisplayName = Item->GetDisplayName();
	}
	else
	{
		Environment = nullptr;
		EnvironmentDisplayName.Reset();
	}

	RefreshAll();
#endif // WITH_MASSENTITY_DEBUG
}

void FMassDebuggerModel::RefreshAll()
{
#if WITH_MASSENTITY_DEBUG
	if (Environment)
	{
		TMap<FMassArchetypeHandle, TSharedPtr<FMassDebuggerArchetypeData>> TransientArchetypesMap;

		CacheArchetypesData(TransientArchetypesMap);
		TArray<TNotNull<const UMassCompositeProcessor*>> CompositeProcessors;
		CacheProcessorsData(TransientArchetypesMap, CompositeProcessors);
		CacheProcessingGraphs(CompositeProcessors);
		CacheFragmentData();
		CacheTagData();
		ClearArchetypeSelection();

		ReconcileAllBreakpoints();
		ApplyBreakpointsToCurrentEnvironment();

		OnRefreshDelegate.Broadcast();
	}
#endif // WITH_MASSENTITY_DEBUG
}

void FMassDebuggerModel::SelectProcessor(TSharedPtr<FMassDebuggerProcessorData>& Processor)
{
	SelectProcessors(MakeArrayView(&Processor, 1), ESelectInfo::Direct);
}

void FMassDebuggerModel::SelectProcessors(TArrayView<TSharedPtr<FMassDebuggerProcessorData>> Processors, ESelectInfo::Type SelectInfo)
{
	SelectionMode = EMassDebuggerSelectionMode::Processor;

	ResetSelectedProcessors();
	ResetSelectedArchetypes();

	SelectedProcessors = Processors;

	for (TSharedPtr<FMassDebuggerProcessorData>& ProcessorData : SelectedProcessors)
	{
		check(ProcessorData.IsValid());
		ProcessorData->Selection = EMassDebuggerProcessorSelection::Selected;

		for (TSharedPtr<FMassDebuggerArchetypeData>& ArchetypeData : ProcessorData->ValidArchetypes)
		{
			SelectedArchetypes.AddUnique(ArchetypeData);
			ArchetypeData->bIsSelected = true;
		}
	}

	OnProcessorsSelectedDelegate.Broadcast(SelectedProcessors, SelectInfo);
}

void FMassDebuggerModel::ClearProcessorSelection()
{
	SelectionMode = EMassDebuggerSelectionMode::None;

	ResetSelectedProcessors();

	OnProcessorsSelectedDelegate.Broadcast(SelectedProcessors, ESelectInfo::Direct);
}

void FMassDebuggerModel::SelectArchetypes(TArrayView<TSharedPtr<FMassDebuggerArchetypeData>> InSelectedArchetypes, ESelectInfo::Type SelectInfo)
{
	ResetSelectedProcessors();
	ResetSelectedArchetypes();

	SelectionMode = EMassDebuggerSelectionMode::Archetype;

	SelectedArchetypes = InSelectedArchetypes;

	for (TSharedPtr<FMassDebuggerProcessorData>& ProcessorData : AllCachedProcessors)
	{
		check(ProcessorData.IsValid());
		for (const TSharedPtr<FMassDebuggerArchetypeData>& ArchetypeData : InSelectedArchetypes)
		{
			if (ProcessorData->ValidArchetypes.Find(ArchetypeData) != INDEX_NONE)
			{
				ProcessorData->Selection = EMassDebuggerProcessorSelection::Selected;
				SelectedProcessors.Add(ProcessorData);
				break;
			}
		}
	}

	OnArchetypesSelectedDelegate.Broadcast(SelectedArchetypes, SelectInfo);
}

void FMassDebuggerModel::ClearArchetypeSelection()
{
	SelectionMode = EMassDebuggerSelectionMode::None;

	ResetSelectedArchetypes();
	OnArchetypesSelectedDelegate.Broadcast(SelectedArchetypes, ESelectInfo::Direct);
}

void FMassDebuggerModel::CacheProcessorsData(const TMap<FMassArchetypeHandle, TSharedPtr<FMassDebuggerArchetypeData>>& InTransientArchetypesMap, TArray<TNotNull<const UMassCompositeProcessor*>>& OutCompositeProcessors)
{
	static auto SortPredicate = [](const TSharedPtr<FMassDebuggerProcessorData>& A, const TSharedPtr<FMassDebuggerProcessorData>& B)
	{
		return A->Label < B->Label;
	};

	CachedProcessorCollections.Reset();
	AllCachedProcessors.Reset();
	ProcessorNames.Reset();
	ProcessorMap.Reset();
	
	if (Environment == nullptr)
	{
		return;
	}

	UWorld* World = Environment->World.Get();

	if (TSharedPtr<const FMassEntityManager> EntityManager = Environment->GetEntityManager())
	{
		// run all the processor providers and convert the data to FMassDebuggerProcessorData
		TArray<const UMassProcessor*> TmpProcessors;
		for (auto& Providers : Environment->ProcessorProviders)
		{
			
			// this will fill TmpProcessors with results of the stored FMassDebugger::FProcessorProviderFunction,
			// the "Value" is of type FMassDebugger::FProcessorProviderFunction
			Providers.Value(TmpProcessors);
			if (TmpProcessors.IsEmpty())
			{
				continue;
			}

			TSharedPtr<FProcessorCollection>& Collection = CachedProcessorCollections.Add_GetRef(MakeShareable(new FProcessorCollection(Providers.Key)));
			TArray<TSharedPtr<FMassDebuggerProcessorData>>& Container = Collection->Container;
			Container.Reserve(TmpProcessors.Num());
			for (const UMassProcessor* Processor : TmpProcessors)
			{
				if (Processor)
				{
					if (const UMassCompositeProcessor* CompositeProcessor = Cast<const UMassCompositeProcessor>(Processor))
					{
						// The composite processors are collected in a dedicated container to get processed
						// in a special way. These processors will be shown in "processing phase" tab.
						// @todo rename the mentioned tab to indicate that it contains all provided composite processors.
						OutCompositeProcessors.Emplace(CompositeProcessor);
					}
					else
					{
						AllCachedProcessors.Add(
							Container.Add_GetRef(MakeShareable(new FMassDebuggerProcessorData(*EntityManager, *Processor, InTransientArchetypesMap)))
							);
					}
				}
			}
			Container.Sort(SortPredicate);

			TmpProcessors.Reset();
		}
	}
	else
	{
		TSharedPtr<FProcessorCollection>& Collection = CachedProcessorCollections.Add_GetRef(MakeShareable(new FProcessorCollection(TEXT("Global view"))));
		TArray<TSharedPtr<FMassDebuggerProcessorData>>& Container = Collection->Container;
		for (FThreadSafeObjectIterator It(UMassProcessor::StaticClass()); It; ++It)
		{
			UMassProcessor* Processor = Cast<UMassProcessor>(*It);
			if (Processor 
				&& Cast<UMassCompositeProcessor>(Processor) == nullptr
				&& UE::Mass::Debugger::Private::IsDebuggableProcessor(World, *Processor))
			{
				Container.Add(MakeShareable(new FMassDebuggerProcessorData(*Processor)));
			}
		}
		Container.Sort(SortPredicate);
	}

	AllCachedProcessors.Sort(SortPredicate);

	for (TSharedPtr<FMassDebuggerProcessorData>& Processor : AllCachedProcessors)
	{
		ProcessorNames.Add(MakeShared<FString>(Processor->Name));
		ProcessorMap.Add(Processor->Name, Processor);
	}
}

void FMassDebuggerModel::CacheProcessingGraphs(TConstArrayView<TNotNull<const UMassCompositeProcessor*>> InCompositeProcessors)
{
	CachedProcessingGraphs.Reset();

	for (TNotNull<const UMassCompositeProcessor*> Processor : InCompositeProcessors)
	{
		CachedProcessingGraphs.Add(MakeShareable(new FMassDebuggerProcessingGraph(*this, Processor)));
	}
}

void FMassDebuggerModel::CacheFragmentData(
	TArray<TSharedPtr<FMassDebuggerFragmentData>>& OutData,
	const TConstArrayView<TWeakObjectPtr<const UScriptStruct>>& FragmentTypes,
	bool bAppend
)
{
	if (!bAppend)
	{
		OutData.Reset();
	}

#if WITH_MASSENTITY_DEBUG
	TMap<const UScriptStruct*, TSharedPtr<FMassDebuggerFragmentData>> TempFragmentDataMap;

	for (const TWeakObjectPtr<const UScriptStruct>& WeakStruct : FragmentTypes)
	{
		if (WeakStruct.IsValid())
		{
			TempFragmentDataMap.Add(WeakStruct.Get(), MakeShared<FMassDebuggerFragmentData>(WeakStruct.Get()));
		}
	}

	if (IsCurrentEnvironmentValid())
	{
		const FMassEntityManager& EntityManager = *Environment->GetEntityManager();
		EntityManager.ForEachArchetype(0, TNumericLimits<int32>::Max(), [this, &TempFragmentDataMap](const FMassEntityManager& EntityManager, const FMassArchetypeHandle& ArchetypeHandle)
		{
			const int32 NumEntitiesInArchetype = EntityManager.DebugGetArchetypeEntitiesCount(ArchetypeHandle);

			EntityManager.ForEachArchetypeFragmentType(ArchetypeHandle, [this, &TempFragmentDataMap, &ArchetypeHandle, NumEntitiesInArchetype](const UScriptStruct* FragmentStruct)
			{
				if (FragmentStruct)
				{
					TSharedPtr<FMassDebuggerFragmentData>* FragmentEntry = TempFragmentDataMap.Find(FragmentStruct);

					if (FragmentEntry && (*FragmentEntry))
					{
						(*FragmentEntry)->Archetypes.AddUnique(ArchetypeHandle);
						(*FragmentEntry)->NumEntities += NumEntitiesInArchetype;
					}
				}
			});
		});
	}

	OutData.Reserve(TempFragmentDataMap.Num());
	for (auto const& [FragmentStruct, FragmentData] : TempFragmentDataMap)
	{
		OutData.Add(FragmentData);
	}

	if (IsCurrentEnvironmentValid())
	{
		TWeakPtr<const FMassEntityManager> EntityManager = Environment->GetEntityManager().ToWeakPtr();
		for (TSharedPtr<FMassDebuggerFragmentData>& FragmentData : OutData)
		{
			FragmentData->EntityManager = EntityManager;
		}
	}

	static auto SortPredicate = [](const TSharedPtr<FMassDebuggerFragmentData>& A, const TSharedPtr<FMassDebuggerFragmentData>& B)
	{
		return A->Name.CompareTo(B->Name) <= 0;
	};
	OutData.Sort(SortPredicate);
#endif
}

void FMassDebuggerModel::CacheFragmentData()
{
#if WITH_MASSENTITY_DEBUG
	CacheFragmentData(CachedFragmentData, FMassFragmentBitSet::DebugGetAllStructTypes(), false);
	CacheFragmentData(CachedFragmentData, FMassChunkFragmentBitSet::DebugGetAllStructTypes(), true);
	CacheFragmentData(CachedFragmentData, FMassSharedFragmentBitSet::DebugGetAllStructTypes(), true);
	CacheFragmentData(CachedFragmentData, FMassConstSharedFragmentBitSet::DebugGetAllStructTypes(), true);
	FragmentNames.Reset();
	FragmentMap.Reset();

	for (TSharedPtr<FMassDebuggerFragmentData>& Fragment : CachedFragmentData)
	{
		FragmentNames.Add(MakeShared<FString>(Fragment->Name.ToString()));
		FragmentMap.Add(Fragment->Name.ToString(), Fragment);
	}
#endif
}

TSharedPtr<FMassDebuggerFragmentData> FMassDebuggerModel::FindFragmentData(const UScriptStruct* FragmentType)
{
#if WITH_MASSENTITY_DEBUG
	for (TSharedPtr<FMassDebuggerFragmentData>& FragmentDataPtr : CachedFragmentData)
	{
		if (FragmentDataPtr->Fragment.Get() == FragmentType)
		{
			return FragmentDataPtr;
		}
	}
#endif
	return nullptr;
}

void FMassDebuggerModel::CacheTagData()
{
#if WITH_MASSENTITY_DEBUG
	CacheFragmentData(CachedTagData, FMassTagBitSet::DebugGetAllStructTypes(), false);
	TagNames.Reset();
	TagMap.Reset();

	for (TSharedPtr<FMassDebuggerFragmentData>& Tag : CachedTagData)
	{
		TagNames.Add(MakeShared<FString>(Tag->Name.ToString()));
		TagMap.Add(Tag->Name.ToString(), Tag);
	}
#endif
}

void FMassDebuggerModel::CacheArchetypesData(TMap<FMassArchetypeHandle, TSharedPtr<FMassDebuggerArchetypeData>>& OutTransientArchetypesMap)
{
#if WITH_MASSENTITY_DEBUG
	CachedAllArchetypes.Reset();
	CachedArchetypeRepresentatives.Reset();

	if (Environment)
	{
		if (TSharedPtr<const FMassEntityManager> EntityManager = Environment->GetEntityManager())
		{
			StoreArchetypes(*EntityManager, OutTransientArchetypesMap);
		}
	}
#endif
}

void FMassDebuggerModel::LoadBreakpointsFromDisk()
{
#if WITH_MASSENTITY_DEBUG
	CachedBreakpoints.Reset();
	FMassDebugger::ClearAllBreakpoints();
	
	const FString SavedDir = FPaths::ProjectSavedDir() / TEXT("MassDebugger");
	const FString FullPath = SavedDir / TEXT("Breakpoints.json");

	if (FPaths::FileExists(FullPath))
	{
		FString FileContent;
		if (FFileHelper::LoadFileToString(FileContent, *FullPath))
		{
			TSharedPtr<FJsonObject> RootObj;
			TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(FileContent);
			if (FJsonSerializer::Deserialize(Reader, RootObj) && RootObj.IsValid())
			{
				const TArray<TSharedPtr<FJsonValue>>* JsonBreakpoints = nullptr;
				if (RootObj->TryGetArrayField(TEXT("Breakpoints"), JsonBreakpoints))
				{
					for (const TSharedPtr<FJsonValue>& Val : *JsonBreakpoints)
					{
						if (TSharedPtr<FJsonObject> Obj = Val->AsObject())
						{
							TSharedPtr<FMassDebuggerBreakpointData> NewBp = MakeShared<FMassDebuggerBreakpointData>();
							if (NewBp->DeserializeFromJson(Obj))
							{
								CachedBreakpoints.Add(NewBp);
							}
						}
					}
				}
			}
		}
	}
	OnEditorBreakpointsChangedDelegate.Broadcast();
#endif //WITH_MASSENTITY_DEBUG
}

void FMassDebuggerModel::SaveBreakpointsToDisk()
{
#if WITH_MASSENTITY_DEBUG
	const FString FileName = TEXT("Breakpoints.json");
	const FString SavedDir = FPaths::ProjectSavedDir() / TEXT("MassDebugger");
	IFileManager::Get().MakeDirectory(*SavedDir, /*Tree=*/true);
	const FString FullPath = SavedDir / FileName;

	TSharedPtr<FJsonObject> RootObject = MakeShared<FJsonObject>();
	TArray<TSharedPtr<FJsonValue>> BreakpointsArray;
	for (TSharedPtr<FMassDebuggerBreakpointData>& Breakpoint : CachedBreakpoints)
	{
		BreakpointsArray.Add(MakeShared<FJsonValueObject>(Breakpoint->SerializeToJson()));
	}
	RootObject->SetArrayField(TEXT("Breakpoints"), MoveTemp(BreakpointsArray));

	FString OutString;
	const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutString);
	FJsonSerializer::Serialize(RootObject.ToSharedRef(), Writer);
	FFileHelper::SaveStringToFile(OutString, *FullPath);
#endif //WITH_MASSENTITY_DEBUG
}

void FMassDebuggerModel::CreateBreakpoint()
{
#if WITH_MASSENTITY_DEBUG
	TSharedPtr<FMassDebuggerBreakpointData> NewBp = MakeShared<FMassDebuggerBreakpointData>();
	CachedBreakpoints.Add(NewBp);
	if (!IsStale())
	{
		NewBp->ApplyToEngine(*this);
	}
	OnEditorBreakpointsChangedDelegate.Broadcast();
#endif //WITH_MASSENTITY_DEBUG
}

void FMassDebuggerModel::CreateBreakpointFromString(const FString& InString)
{
#if WITH_MASSENTITY_DEBUG
	TSharedPtr<FMassDebuggerBreakpointData> NewBp = MakeShared<FMassDebuggerBreakpointData>();
	if (NewBp->DeserializeFromJsonString(InString))
	{
		CachedBreakpoints.Add(NewBp);
		if (!IsStale())
		{
			NewBp->ApplyToEngine(*this);
		}
		OnEditorBreakpointsChangedDelegate.Broadcast();
	}
#endif //WITH_MASSENTITY_DEBUG
}

void FMassDebuggerModel::ApplyBreakpointsToCurrentEnvironment()
{
#if WITH_MASSENTITY_DEBUG
	if (!IsStale())
	{
		FMassDebugger::ClearAllBreakpoints();
		const FMassEntityManager& EntityManager = *Environment->EntityManager.Pin();
		for (TSharedPtr<FMassDebuggerBreakpointData>& Breakpoint : CachedBreakpoints)
		{
			// apply breakpoint but wait until all are added before refreshing
			Breakpoint->ApplyToEngine(*this, false);
		}
		FMassDebugger::RefreshBreakpoints();
	}
	OnEditorBreakpointsChangedDelegate.Broadcast();
#endif //WITH_MASSENTITY_DEBUG
}

void FMassDebuggerModel::ReconcileAllBreakpoints()
{
#if WITH_MASSENTITY_DEBUG
	for (TSharedPtr<FMassDebuggerBreakpointData>& Breakpoint : CachedBreakpoints)
	{
		Breakpoint->ReconcileDataFromNames(*this);
	}
	OnEditorBreakpointsChangedDelegate.Broadcast();
#endif //WITH_MASSENTITY_DEBUG
}

void FMassDebuggerModel::RemoveAllBreakpoints()
{
#if WITH_MASSENTITY_DEBUG
	CachedBreakpoints.Reset();
	ApplyBreakpointsToCurrentEnvironment();
#endif //WITH_MASSENTITY_DEBUG
}

void FMassDebuggerModel::RemoveBreakpoint(UE::Mass::Debug::FBreakpointHandle Handle)
{
#if WITH_MASSENTITY_DEBUG
	for (int32 i = 0; i < CachedBreakpoints.Num(); i++)
	{
		if (CachedBreakpoints[i]->BreakpointInstance.Handle == Handle)
		{
			CachedBreakpoints.RemoveAt(i);
			ApplyBreakpointsToCurrentEnvironment();
			return;
		}
	}
#endif //WITH_MASSENTITY_DEBUG
}

FMassDebuggerBreakpointData::FMassDebuggerBreakpointData(const UE::Mass::Debug::FBreakpoint& InBreakpoint, FMassDebuggerModel& Model)
{
#if WITH_MASSENTITY_DEBUG
	BreakpointInstance = InBreakpoint;
	ReconcileDataFromEngineBreakpoint(Model);
#endif //WITH_MASSENTITY_DEBUG
}

void FMassDebuggerBreakpointData::ReconcileDataFromNames(FMassDebuggerModel& Model)
{
#if WITH_MASSENTITY_DEBUG
	switch (BreakpointInstance.TriggerType)
	{
	case UE::Mass::Debug::FBreakpoint::ETriggerType::ProcessorExecute:
	{
		if (TWeakPtr<FMassDebuggerProcessorData>* Found = Model.ProcessorMap.Find(TriggerName))
		{
			if ((TriggerProcessor = (*Found).Pin()))
			{
				if (const UMassProcessor* Processor = TriggerProcessor->Processor.Get())
				{
					BreakpointInstance.Trigger.Set<TObjectKey<const UMassProcessor>>(Processor);
				}
			}
		}
		break;
	}
	case UE::Mass::Debug::FBreakpoint::ETriggerType::FragmentWrite:
	case UE::Mass::Debug::FBreakpoint::ETriggerType::FragmentAdd:
	case UE::Mass::Debug::FBreakpoint::ETriggerType::FragmentRemove:
	case UE::Mass::Debug::FBreakpoint::ETriggerType::TagAdd:
	case UE::Mass::Debug::FBreakpoint::ETriggerType::TagRemove:
	{
		if (TWeakPtr<FMassDebuggerFragmentData>* Found = Model.FragmentMap.Find(TriggerName))
		{
			if ((TriggerFragment = (*Found).Pin()))
			{
				if (const UScriptStruct* Fragment = TriggerFragment->Fragment.Get())
				{
					BreakpointInstance.Trigger.Set<TObjectKey<const UScriptStruct>>(Fragment);
				}
			}
		}
		break;
	}
	default:
		break;
	}
	
	if(BreakpointInstance.FilterType == UE::Mass::Debug::FBreakpoint::EFilterType::Query)
	{
		if (TWeakPtr<UE::MassDebugger::FEditableQuery>* Found = Model.QueryMap.Find(FilterName))
		{
			FilterQuery = (*Found).Pin();
			if (!Model.IsStale())
			{
				TSharedPtr<FMassEntityManager> EntityManager = Model.Environment->GetMutableEntityManager();
				BreakpointInstance.Filter.Set<FMassFragmentRequirements>(FilterQuery->BuildEntityQuery(EntityManager.ToSharedRef()));
			}
		}
	}
#endif //WITH_MASSENTITY_DEBUG
}

void FMassDebuggerBreakpointData::ReconcileDataFromEngineBreakpoint(FMassDebuggerModel& Model)
{
#if WITH_MASSENTITY_DEBUG
	switch (BreakpointInstance.TriggerType)
	{
	case UE::Mass::Debug::FBreakpoint::ETriggerType::ProcessorExecute:
		if (BreakpointInstance.Trigger.IsType<TObjectKey<const UMassProcessor>>())
		{
			const UMassProcessor* Proc = BreakpointInstance.Trigger.Get<TObjectKey<const UMassProcessor>>().ResolveObjectPtr();

			if (Proc)
			{
				TriggerProcessor = Model.GetProcessorDataChecked(*Proc);
				TriggerName = TriggerProcessor->Name;
			}
		}
		break;

	case UE::Mass::Debug::FBreakpoint::ETriggerType::FragmentWrite:
	case UE::Mass::Debug::FBreakpoint::ETriggerType::FragmentAdd:
	case UE::Mass::Debug::FBreakpoint::ETriggerType::FragmentRemove:
	case UE::Mass::Debug::FBreakpoint::ETriggerType::TagAdd:
	case UE::Mass::Debug::FBreakpoint::ETriggerType::TagRemove:

		if (BreakpointInstance.Trigger.IsType<TObjectKey<const UScriptStruct>>())
		{
			const UScriptStruct* StructType = BreakpointInstance.Trigger.Get<TObjectKey<const UScriptStruct>>().ResolveObjectPtr();
			if (StructType)
			{
				if ((TriggerFragment = Model.FindFragmentData(StructType)))
				{
					TriggerName = TriggerFragment->Name.ToString();
				}
			}
		}
		break;

	default:
		break;
	}
#endif //WITH_MASSENTITY_DEBUG
}

void FMassDebuggerBreakpointData::ApplyToEngine(FMassDebuggerModel& Model, bool bRefreshEngineBreakpoints/* = true*/)
{
#if WITH_MASSENTITY_DEBUG
	if (Model.IsStale())
	{
		return;
	}
	
	TSharedPtr<FMassEntityManager> EntityManager = Model.Environment->GetMutableEntityManager();

	if (FMassDebugger::FEnvironment* Environment = FMassDebugger::FindEnvironmentForEntityManager(*EntityManager))
	{
		if (BreakpointInstance.FilterType == UE::Mass::Debug::FBreakpoint::EFilterType::Query
			&& !BreakpointInstance.Filter.IsType<FMassFragmentRequirements>()
			&& FilterQuery.IsValid())
		{
			BreakpointInstance.Filter.Set<FMassFragmentRequirements>(FilterQuery->BuildEntityQuery(EntityManager.ToSharedRef()));
		}

		if (UE::Mass::Debug::FBreakpoint* Breakpoint = Environment->FindBreakpointByHandle(BreakpointInstance.Handle))
		{
			*Breakpoint = BreakpointInstance;
		}
		else
		{
			UE::Mass::Debug::FBreakpoint& NewBreakpoint = FMassDebugger::CreateBreakpoint(*EntityManager);
			NewBreakpoint = BreakpointInstance;
		}
		if (bRefreshEngineBreakpoints)
		{
			FMassDebugger::RefreshBreakpoints();
		}
	}
#endif
}

TSharedPtr<FJsonObject> FMassDebuggerBreakpointData::SerializeToJson() const
{
	TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();

#if WITH_MASSENTITY_DEBUG
	JsonObject->SetBoolField(TEXT("Enabled"), BreakpointInstance.bEnabled);
	JsonObject->SetStringField(TEXT("TriggerType"), UE::Mass::Debug::FBreakpoint::TriggerTypeToString(BreakpointInstance.TriggerType));
	JsonObject->SetStringField(TEXT("FilterType"),UE::Mass::Debug::FBreakpoint::FilterTypeToString(BreakpointInstance.FilterType));

	switch (BreakpointInstance.TriggerType)
	{
	case UE::Mass::Debug::FBreakpoint::ETriggerType::ProcessorExecute:
		if (TriggerProcessor.IsValid())
		{
			JsonObject->SetStringField(TEXT("ProcessorName"),TriggerProcessor->Name);
		}
		else
		{
			JsonObject->SetStringField(TEXT("ProcessorName"), TriggerName);
		}
		break;

	case UE::Mass::Debug::FBreakpoint::ETriggerType::FragmentWrite:
	case UE::Mass::Debug::FBreakpoint::ETriggerType::FragmentAdd:
	case UE::Mass::Debug::FBreakpoint::ETriggerType::FragmentRemove:
	case UE::Mass::Debug::FBreakpoint::ETriggerType::TagAdd:
	case UE::Mass::Debug::FBreakpoint::ETriggerType::TagRemove:
		if (TriggerFragment.IsValid())
		{
			JsonObject->SetStringField(TEXT("FragmentName"),TriggerFragment->Name.ToString());
		}
		else
		{
			JsonObject->SetStringField(TEXT("FragmentName"), TriggerName);
		}
		break;

	default:
		break;
	}

	switch (BreakpointInstance.FilterType)
	{
	case UE::Mass::Debug::FBreakpoint::EFilterType::SpecificEntity:
	{
		const FMassEntityHandle& Handle = BreakpointInstance.Filter.Get<FMassEntityHandle>();
		JsonObject->SetNumberField(TEXT("EntityIndex"), Handle.Index);
		JsonObject->SetNumberField(TEXT("EntitySerialNumber"), Handle.SerialNumber);
		break;
	}
	case UE::Mass::Debug::FBreakpoint::EFilterType::Query:
		if (FilterQuery.IsValid())
		{
			JsonObject->SetStringField(TEXT("FilterQueryName"),FilterQuery->Name);
		}
		else
		{
			JsonObject->SetStringField(TEXT("FilterQueryName"), FilterName);
		}
		break;

	default:
		break;
	}
#endif //WITH_MASSENTITY_DEBUG
	return JsonObject;
}

bool FMassDebuggerBreakpointData::DeserializeFromJson(const TSharedPtr<FJsonObject>& JsonObject)
{
#if WITH_MASSENTITY_DEBUG
	if (!JsonObject.IsValid())
	{
		return false;
	}

	BreakpointInstance.bEnabled = false;
	JsonObject->TryGetBoolField(TEXT("Enabled"), BreakpointInstance.bEnabled);

	{
		FString TriggerTypeStr;
		if (JsonObject->TryGetStringField(TEXT("TriggerType"), TriggerTypeStr))
		{
			UE::Mass::Debug::FBreakpoint::StringToTriggerType(TriggerTypeStr, BreakpointInstance.TriggerType);
		}
	}

	TriggerProcessor.Reset();
	TriggerFragment.Reset();

	switch (BreakpointInstance.TriggerType)
	{
	case UE::Mass::Debug::FBreakpoint::ETriggerType::ProcessorExecute:
	{
		if (!JsonObject->TryGetStringField(TEXT("ProcessorName"), TriggerName))
		{
			return false;
		}
		break;
	}
	case UE::Mass::Debug::FBreakpoint::ETriggerType::FragmentWrite:
	case UE::Mass::Debug::FBreakpoint::ETriggerType::FragmentAdd:
	case UE::Mass::Debug::FBreakpoint::ETriggerType::FragmentRemove:
	case UE::Mass::Debug::FBreakpoint::ETriggerType::TagAdd:
	case UE::Mass::Debug::FBreakpoint::ETriggerType::TagRemove:
	{
		if (!JsonObject->TryGetStringField(TEXT("FragmentName"), TriggerName))
		{
			return false;
		}
		break;
	}
	default:
		break;
	}

	{
		FString FilterTypeStr;
		if (JsonObject->TryGetStringField(TEXT("FilterType"), FilterTypeStr))
		{
			UE::Mass::Debug::FBreakpoint::StringToFilterType(FilterTypeStr, BreakpointInstance.FilterType);
		}
	}

	FilterQuery.Reset();

	switch (BreakpointInstance.FilterType)
	{
	case UE::Mass::Debug::FBreakpoint::EFilterType::SpecificEntity:
	{
		int32 EntityIndex = JsonObject->GetIntegerField(TEXT("EntityIndex"));
		int32 EntitySerialNumber = JsonObject->GetIntegerField(TEXT("EntitySerialNumber"));
		BreakpointInstance.Filter.Set<FMassEntityHandle>(FMassEntityHandle(EntityIndex, EntitySerialNumber));
		break;
	}
	case UE::Mass::Debug::FBreakpoint::EFilterType::Query:
	{
		if (!JsonObject->TryGetStringField(TEXT("FilterQueryName"), FilterName))
		{
			return false;
		}
		break;
	}
	default:
		break;
	}

	return true;
#else
	return false;
#endif //WITH_MASSENTITY_DEBUG
}

bool FMassDebuggerBreakpointData::DeserializeFromJsonString(const FString& JsonString)
{
#if WITH_MASSENTITY_DEBUG
	TSharedPtr<FJsonObject> JsonObject;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(JsonString);

	if (!FJsonSerializer::Deserialize(Reader, JsonObject) || !JsonObject.IsValid())
	{
		return false;
	}

	return DeserializeFromJson(JsonObject);
#else
	return false;
#endif //WITH_MASSENTITY_DEBUG
}


float FMassDebuggerModel::MinDistanceToSelectedArchetypes(const TSharedPtr<FMassDebuggerArchetypeData>& InArchetypeData) const
{
	float MinDistance = MAX_flt;
	for (const TSharedPtr<FMassDebuggerArchetypeData>& SelectedArchetype : SelectedArchetypes)
	{
		MinDistance = FMath::Min(MinDistance, ArchetypeDistances[SelectedArchetype->Index][InArchetypeData->Index]);
	}
	return MinDistance;
}

void FMassDebuggerModel::StoreArchetypes(const FMassEntityManager& EntityManager, TMap<FMassArchetypeHandle, TSharedPtr<FMassDebuggerArchetypeData>>& OutTransientArchetypesMap)
{
#if WITH_MASSENTITY_DEBUG
	TArray<FMassArchetypeHandle> ArchetypeHandles = FMassDebugger::GetAllArchetypes(EntityManager);

	CachedAllArchetypes.Reset(ArchetypeHandles.Num());

	int32 MaxBitsUsed = 0;

	// @todo build an archetype handle map
	for (FMassArchetypeHandle& ArchetypeHandle : ArchetypeHandles)
	{
		FMassDebuggerArchetypeData* ArchetypeDataPtr = new FMassDebuggerArchetypeData(ArchetypeHandle);
		ArchetypeDataPtr->Index = CachedAllArchetypes.Add(MakeShareable(ArchetypeDataPtr));
		OutTransientArchetypesMap.Add(ArchetypeHandle, CachedAllArchetypes.Last());

		MaxBitsUsed = FMath::Max(MaxBitsUsed, ArchetypeDataPtr->GetTotalBitsUsedCount());
	}
#endif // WITH_MASSENTITY_DEBUG

	// calculate distances
	ArchetypeDistances.Reset();
	ArchetypeDistances.AddDefaulted(CachedAllArchetypes.Num());
	for (int i = 0; i < CachedAllArchetypes.Num(); ++i)
	{
		ArchetypeDistances[i].AddDefaulted(CachedAllArchetypes.Num());
	}

	for (int i = 0; i < CachedAllArchetypes.Num(); ++i)
	{
		for (int k = i + 1; k < CachedAllArchetypes.Num(); ++k)
		{
			const float Distance = UE::Mass::Debugger::Private::CalcArchetypeBitDistance(*CachedAllArchetypes[i].Get(), *CachedAllArchetypes[k].Get());
			ArchetypeDistances[i][k] = Distance;
			ArchetypeDistances[k][i] = Distance;
		}
	}

	// Add archetypes that share same primary name under the same entry. 
	TMap<FString, TSharedPtr<FMassDebuggerArchetypeData>> ArchetypeNameMap;
	for (TSharedPtr<FMassDebuggerArchetypeData>& ArchetypeData : CachedAllArchetypes)
	{
		if (const TSharedPtr<FMassDebuggerArchetypeData>* Representative = ArchetypeNameMap.Find(ArchetypeData->PrimaryDebugName))
		{
			(*Representative)->Children.Add(ArchetypeData);
			ArchetypeData->Parent = *Representative;
		}
		else
		{
			ArchetypeNameMap.Add(ArchetypeData->PrimaryDebugName, ArchetypeData);
		}
	}

	for (auto& KeyValue : ArchetypeNameMap)
	{
		CachedArchetypeRepresentatives.Add(KeyValue.Value);
	}
}

FText FMassDebuggerModel::GetDisplayName() const
{
	if (!Environment)
	{
		return LOCTEXT("PickEnvironment", "Pick Environment");
	}
	else if (IsStale())
	{
		return FText::FromString(FString::Printf(TEXT("(%s) %s")
				, *(LOCTEXT("StaleEnvironmentPrefix", "Stale").ToString())
				, *EnvironmentDisplayName));
	}

	return FText::FromString(Environment->GetDisplayName());
}

void FMassDebuggerModel::MarkAsStale()
{
	if (Environment)
	{
		Environment->World = nullptr;
	}
}

bool FMassDebuggerModel::IsStale() const 
{ 
#if WITH_MASSENTITY_DEBUG
	return Environment.IsValid() == false
		|| Environment->EntityManager.IsValid() == false
		|| FMassDebugger::IsEntityManagerInitialized(*Environment->EntityManager.Pin()) == false
		|| (Environment->NeedsValidWorld() == true && Environment->IsWorldValid() == false);
#else
	return true;
#endif //WITH_MASSENTITY_DEBUG
}

const TSharedPtr<FMassDebuggerProcessorData>& FMassDebuggerModel::GetProcessorDataChecked(const UMassProcessor& Processor) const
{
	check(AllCachedProcessors.Num());

	const uint32 ProcessorHash = UE::Mass::Debugger::Private::CalcProcessorHash(Processor);
	auto SearchPredicate = [ProcessorHash](const TSharedPtr<FMassDebuggerProcessorData>& Element)
		{
			return Element->ProcessorHash == ProcessorHash;
		};

	// @todo could convert AllCachedProcessors to a map if this search becomes too slow
	const TSharedPtr<FMassDebuggerProcessorData>* DataFound = AllCachedProcessors.FindByPredicate(SearchPredicate);

	check(DataFound);
	return *DataFound;
}

void FMassDebuggerModel::RegisterEntitiesView(TSharedRef<SMassEntitiesView> EntitiesView, int32 Index)
{
	if (EntityViews.Num() < (Index + 1))
	{
		EntityViews.SetNum(Index + 1);
	}
	EntityViews[Index] = EntitiesView;
}

void FMassDebuggerModel::ShowEntitiesView(int Index, FMassArchetypeHandle ArchetypeHandle)
{
	TWeakPtr<SMassEntitiesView> View = ShowEntitiesView(Index);
	if (!View.IsValid())
	{
		return;
	}
	View.Pin()->ShowEntities(ArchetypeHandle);
}

void FMassDebuggerModel::ShowEntitiesView(int Index, TArray<FMassEntityHandle> EntitieHandles)
{
	TWeakPtr<SMassEntitiesView> View = ShowEntitiesView(Index);
	if (!View.IsValid())
	{
		return;
	}
	View.Pin()->ShowEntities(EntitieHandles);
}

void FMassDebuggerModel::ShowEntitiesView(int Index, FMassEntityQuery& Query)
{
	TWeakPtr<SMassEntitiesView> View = ShowEntitiesView(Index);
	if (!View.IsValid())
	{
		return;
	}
	View.Pin()->ShowEntities(Query);
}

void FMassDebuggerModel::ShowEntitiesView(int Index, TConstArrayView<FMassEntityQuery*> InQueries)
{
	TWeakPtr<SMassEntitiesView> View = ShowEntitiesView(Index);
	if (!View.IsValid())
	{
		return;
	}
	View.Pin()->ShowEntities(InQueries);
}

TWeakPtr<SMassEntitiesView> FMassDebuggerModel::ShowEntitiesView(int32 Index)
{
	if (DebuggerWindow.IsValid())
	{
		DebuggerWindow.Pin()->ShowEntitesView();
	}

	check(Index < MaxEntityViewCount);
	if (EntityViews.Num() < (Index + 1))
	{
		// TODO: create the tab and set focus to it
	}
	return EntityViews[Index];
}

void FMassDebuggerModel::ResetEntitiesViews()
{
	for (int i = 0; i < MaxEntityViewCount; i++)
	{
		if (EntityViews[i].IsValid())
		{
			EntityViews[i].Pin()->ClearEntities();
		}
	}
}

void FMassDebuggerModel::RegisterQueryEditor(TSharedRef<UE::MassDebugger::SQueryEditorView> InQueryEditorView)
{
	QueryEditorView = InQueryEditorView;
}

void FMassDebuggerModel::ShowQueryInEditor(const FMassEntityQuery& InQuery, const FString& InQueryName)
{
	if (TSharedPtr<SMassDebugger> SharedDebugger = DebuggerWindow.Pin())
	{
		SharedDebugger->ShowQueryEditorView();
	}

	if (TSharedPtr<UE::MassDebugger::SQueryEditorView> SharedEditor = QueryEditorView.Pin())
	{
		SharedEditor->ShowQuery(InQuery, InQueryName);
	}
}

void FMassDebuggerModel::ResetSelectedArchetypes()
{
	for (TSharedPtr<FMassDebuggerArchetypeData>& ArchetypeData : SelectedArchetypes)
	{
		ArchetypeData->bIsSelected = false;
	}
	SelectedArchetypes.Reset();
}

void FMassDebuggerModel::ResetSelectedProcessors()
{
	// resetting marking of all the processors instead of SelectedProcessors to be on the safe side
	for (TSharedPtr<FMassDebuggerProcessorData>& ProcessorData : AllCachedProcessors)
	{
		check(ProcessorData.IsValid());
		ProcessorData->Selection = EMassDebuggerProcessorSelection::None;
	}
	SelectedProcessors.Reset();
}

void FMassDebuggerModel::OnEntitySelected(const FMassEntityManager& EntityManager, const FMassEntityHandle EntityHandle)
{
	if (!Environment || Environment->GetEntityManager().Get() != &EntityManager)
	{
		// not the entity manager we're debugging right now
		return;
	}
	
	const FMassArchetypeHandle ArchetypeHandle = EntityManager.GetArchetypeForEntity(EntityHandle);
	if (ArchetypeHandle.IsValid() == false)
	{
		return;
	}

#if WITH_MASSENTITY_DEBUG
	const uint32 ArchetypeHash = FMassDebugger::GetArchetypeComposition(ArchetypeHandle).CalculateHash();
#else
	const uint32 ArchetypeHash = 0;
#endif // WITH_MASSENTITY_DEBUG
	TSharedPtr<FMassDebuggerArchetypeData>* DebuggerArchetypeData = CachedAllArchetypes.FindByPredicate([ArchetypeHash](const TSharedPtr<FMassDebuggerArchetypeData>& Element)
		{
			return Element.IsValid() && Element->CompositionHash == ArchetypeHash;
		});

	if (DebuggerArchetypeData)
	{
		SelectArchetypes(MakeArrayView(DebuggerArchetypeData, 1), ESelectInfo::Direct);
	}
}

void FMassDebuggerModel::OnBreakpointsChanged()
{
#if WITH_MASSENTITY_DEBUG
	if (IsStale())
	{
		return;
	}
	if (TSharedPtr<const FMassEntityManager> EntityManager = Environment->GetEntityManager())
	{
		for (int i = CachedBreakpoints.Num() - 1; i >= 0; i--)
		{
			TSharedPtr<FMassDebuggerBreakpointData>& BreakData = CachedBreakpoints[i];
			UE::Mass::Debug::FBreakpoint* EngineBreakpoint = FMassDebugger::FindBreakpoint(*EntityManager, BreakData->BreakpointInstance.Handle);
			if (EngineBreakpoint)
			{
				BreakData->BreakpointInstance = *EngineBreakpoint;
			}
		}
		TArray<UE::Mass::Debug::FBreakpoint>& Breakpoints = FMassDebugger::GetBreakpoints(*EntityManager);
		for (UE::Mass::Debug::FBreakpoint& EngineBreakpoint : Breakpoints)
		{
			bool Exists = Algo::AnyOf(CachedBreakpoints, [InId = EngineBreakpoint.Handle](const TSharedPtr<FMassDebuggerBreakpointData>& BpData)
			{
				return BpData->BreakpointInstance.Handle == InId;
			});
			if (!Exists)
			{
				TSharedPtr<FMassDebuggerBreakpointData> NewBp = MakeShared<FMassDebuggerBreakpointData>(EngineBreakpoint, *this);
				CachedBreakpoints.Add(NewBp);
			}
		}
		OnEditorBreakpointsChangedDelegate.Broadcast();
	}
#endif //WITH_MASSENTITY_DEBUG
}

void FMassDebuggerModel::SelectFragment(FName InFragementName)
{
	SelectedFragmentName = InFragementName;
	OnFragmentSelectedDelegate.Broadcast(SelectedFragmentName);
}

FName FMassDebuggerModel::GetSelectedFragment()
{
	return SelectedFragmentName;
}

void FMassDebuggerModel::LoadQueriesFromDisk()
{
	const FString SavedDir = FPaths::ProjectSavedDir() / TEXT("MassDebugger");
	const FString FullPath = SavedDir / TEXT("Queries.json");

	QueryList.Empty();

	if (FPaths::FileExists(FullPath))
	{
		FString FileContent;
		if (FFileHelper::LoadFileToString(FileContent, *FullPath))
		{
			TSharedPtr<FJsonObject> RootObj;
			TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(FileContent);
			if (FJsonSerializer::Deserialize(Reader, RootObj) && RootObj.IsValid())
			{
				const TArray<TSharedPtr<FJsonValue>>* JsonQueries = nullptr;
				if (RootObj->TryGetArrayField(TEXT("Queries"), JsonQueries))
				{
					for (const auto& Val : *JsonQueries)
					{
						if (TSharedPtr<FJsonObject> Obj = Val->AsObject())
						{
							TSharedPtr<UE::MassDebugger::FEditableQuery> NewQ = MakeShared<UE::MassDebugger::FEditableQuery>();
							if (NewQ->DeserializeFromJson(Obj))
							{
								QueryNames.Add(MakeShared<FString>(NewQ->Name));
								QueryMap.Add(NewQ->Name, NewQ);
								QueryList.Add(MoveTemp(NewQ));
							}
						}
					}
				}
			}
		}
	}
	RefreshQueries();
}

void FMassDebuggerModel::RefreshQueries()
{
	// this should never happen, but clean up here to be safe and avoid having to check below
	for (int32 Index = QueryList.Num() - 1; Index >= 0; --Index)
	{
		if (!QueryList[Index].IsValid())
		{
			QueryList.RemoveAt(Index);
		}
	}

	// Clean up duplicate names:
	const int32 Count = QueryList.Num();

	TArray<FString> OriginalNames;
	OriginalNames.Reserve(Count);
	for (const TSharedPtr<UE::MassDebugger::FEditableQuery>& QueryPtr : QueryList)
	{
		OriginalNames.Add(QueryPtr.IsValid() ? QueryPtr->Name : FString());
	}

	TArray<bool> Collides;
	Collides.Init(false, Count);
	TMap<FString, int32> MaxSuffix;

	for (int32 i = 0; i < Count; ++i)
	{
		const FString& Name = OriginalNames[i];
		FString Base = Name;
		int32 UnderscoreIdx;
		if (Name.FindLastChar(TEXT('_'), UnderscoreIdx))
		{
			const FString Sfx = Name.Mid(UnderscoreIdx + 1);
			if (Sfx.IsNumeric())
			{
				Base = Name.Left(UnderscoreIdx);
				MaxSuffix.FindOrAdd(Base) = FMath::Max(MaxSuffix.FindOrAdd(Base), FCString::Atoi(*Sfx));
			}
		}
		else
		{
			MaxSuffix.FindOrAdd(Base);
		}

		for (int32 j = 0; j < i; ++j)
		{
			if (OriginalNames[j].Compare(Name, ESearchCase::IgnoreCase) == 0)
			{
				Collides[i] = true;
				break;
			}
		}
	}

	for (int32 i = 0; i < Count; ++i)
	{
		if (!Collides[i] || !QueryList[i].IsValid())
		{
			QueryList[i]->Name = OriginalNames[i];
			continue;
		}

		const FString& Name = OriginalNames[i];
		FString Base = Name;
		int32 UnderscoreIdx;
		if (Name.FindLastChar(TEXT('_'), UnderscoreIdx))
		{
			const FString Sfx = Name.Mid(UnderscoreIdx + 1);
			if (Sfx.IsNumeric())
			{
				Base = Name.Left(UnderscoreIdx);
			}
		}
		int32& Next = MaxSuffix.FindOrAdd(Base);
		const FString NewName = FString::Printf(TEXT("%s_%d"), *Base, ++Next);
		QueryList[i]->Name = NewName;
	}

	// sync up query name list
	// This could be done more simply by just rebuilding the name list, but we want
	// to avoid reallocating string ptrs to play nicely with combo box selections.
	TArray<TSharedPtr<FString>> NewQueryNames;
	NewQueryNames.Reserve(QueryList.Num());

	for (const TSharedPtr<UE::MassDebugger::FEditableQuery>& QueryPtr : QueryList)
	{
		const FString& DesiredName = QueryPtr->Name;

		// ordering might not be consistent so look through the entire list
		TSharedPtr<FString> ExistingNamePtr;
		for (const TSharedPtr<FString>& NamePtr : QueryNames)
		{
			if (NamePtr.IsValid() && *NamePtr == DesiredName)
			{
				ExistingNamePtr = NamePtr;
				break;
			}
		}

		if (ExistingNamePtr.IsValid())
		{
			NewQueryNames.Add(ExistingNamePtr);
		}
		else
		{
			NewQueryNames.Add(MakeShared<FString>(DesiredName));
		}
	}

	QueryNames = MoveTemp(NewQueryNames);

	TSet<FString> ValidNames;
	ValidNames.Reserve(QueryList.Num());
	for (const TSharedPtr<UE::MassDebugger::FEditableQuery>& QueryPtr : QueryList)
	{
		ValidNames.Add(QueryPtr->Name);
	}

	for (auto It = QueryMap.CreateIterator(); It; ++It)
	{
		if (!ValidNames.Contains(It->Key))
		{
			It.RemoveCurrent();
		}
	}

	for (const TSharedPtr<UE::MassDebugger::FEditableQuery>& QueryPtr : QueryList)
	{
		QueryMap.Add(QueryPtr->Name, QueryPtr);
	}

	OnQueriesChangedDelegate.Broadcast();
}

namespace UE::MassDebugger
{
	void FEditableQuery::InitFromEntityQuery(const FMassEntityQuery& InQuery, FMassDebuggerModel& DebuggerModel)
	{
		FragmentRequirements.Reset();

		for (const FMassFragmentRequirementDescription& Desc : InQuery.GetFragmentRequirements())
		{
			TSharedPtr<FFragmentEntry> Entry = MakeShared<FFragmentEntry>();
			Entry->StructType = Desc.StructType;
			Entry->AccessMode = Desc.AccessMode;
			Entry->Presence = Desc.Presence;
			FragmentRequirements.Add(Entry);
		}

		const FMassTagBitSet& AllTags = InQuery.GetRequiredAllTags();
		const FMassTagBitSet& AnyTags = InQuery.GetRequiredAnyTags();
		const FMassTagBitSet& OptionalTags = InQuery.GetRequiredOptionalTags();
		const FMassTagBitSet& NoneTags = InQuery.GetRequiredNoneTags();

		for (const TSharedPtr<FMassDebuggerFragmentData>& TagData : DebuggerModel.CachedTagData)
		{
			const UScriptStruct* StructType = TagData->Fragment.Get();
			EMassFragmentPresence Presence = EMassFragmentPresence::All;

			if (AllTags.Contains(StructType))
			{
				Presence = EMassFragmentPresence::All;
			}
			else if (AnyTags.Contains(StructType))
			{
				Presence = EMassFragmentPresence::Any;
			}
			else if (OptionalTags.Contains(StructType))
			{
				Presence = EMassFragmentPresence::Optional;
			}
			else if (NoneTags.Contains(StructType))
			{
				Presence = EMassFragmentPresence::None;
			}
			else
			{
				continue;
			}

			TSharedPtr<FFragmentEntry> Entry = MakeShared<FFragmentEntry>();
			Entry->StructType = StructType;
			Entry->Presence = Presence;
			TagRequirements.Add(Entry);
		}
	}

	FMassEntityQuery FEditableQuery::BuildEntityQuery(const TSharedRef<FMassEntityManager> EntityManager)
	{
		FMassEntityQuery Query;

		Query.Initialize(EntityManager);

		TSet<const UScriptStruct*> UsedFragments;
		for (TSharedPtr<FFragmentEntry>& Fragment : FragmentRequirements)
		{
			const UScriptStruct* RawStruct = Fragment->StructType.Get();
			if (ensure(RawStruct) && !UsedFragments.Contains(RawStruct))
			{
				UsedFragments.Add(RawStruct);
				Query.AddRequirement(RawStruct, Fragment->AccessMode, Fragment->Presence);
			}
		}

		UsedFragments.Reset();
		for (TSharedPtr<FFragmentEntry>& TagEntry : TagRequirements)
		{
			const UScriptStruct* RawStruct = TagEntry->StructType.Get();
			if (ensure(RawStruct) && !UsedFragments.Contains(RawStruct))
			{
				UsedFragments.Add(RawStruct);
				Query.AddTagRequirement(*RawStruct, TagEntry->Presence);
			}
		}
		return Query;
	}

	TSharedPtr<FJsonObject> FFragmentEntry::SerializeToJson() const
	{
		if (!StructType.IsValid())
		{
			return nullptr;
		}
		TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();

		JsonObject->SetStringField(TEXT("StructType"), StructType->GetPathName());
		JsonObject->SetStringField(TEXT("AccessMode"), StaticEnum<EMassFragmentAccess>()->GetNameStringByValue((int64)AccessMode));
		JsonObject->SetStringField(TEXT("Presence"), StaticEnum<EMassFragmentPresence>()->GetNameStringByValue((int64)Presence));

		return JsonObject;
	}

	bool FFragmentEntry::DeserializeFromJson(const TSharedPtr<FJsonObject>& JsonObject)
	{
		if (!JsonObject.IsValid())
		{
			return false;
		}
		FString StructPath = JsonObject->GetStringField(TEXT("StructType"));
		if (StructPath.IsEmpty())
		{
			return false;
		}

		// Honor struct redirects so JSON authored against an older struct path (e.g. before a USTRUCT
		// was moved between modules) still resolves. FindObject/LoadObject only consult Type_Package
		// redirects internally; Type_Struct must be applied explicitly here. Mirrors the pattern in
		// FInstancedStruct::ImportTextItem.
		{
			const FCoreRedirectObjectName OldName(StructPath);
			const FCoreRedirectObjectName NewName = FCoreRedirects::GetRedirectedName(
				ECoreRedirectFlags::Type_Struct, OldName, ECoreRedirectMatchFlags::AllowPartialMatch);
			if (OldName != NewName)
			{
				StructPath = NewName.ToString();
			}
		}

		// Validate before LoadObject to suppress the warning it emits internally on malformed paths
		// (clipboard JSON can carry arbitrary strings).
		if (!FPackageName::IsValidObjectPath(StructPath))
		{
			UE_LOGF(LogMass, Warning, "Mass Debugger: invalid struct path '%ls' - entry skipped.", *StructPath);
			return false;
		}

		// LoadObject (rather than FindObject) so the new struct's module is loaded if it isn't already
		// — important when a redirect points to a package the editor session hasn't touched yet.
		UScriptStruct* const Found = LoadObject<UScriptStruct>(nullptr, *StructPath);
		if (Found == nullptr)
		{
			UE_LOGF(LogMass, Warning, "Mass Debugger: cannot resolve struct type '%ls' - entry skipped.", *StructPath);
			return false;
		}

		const FString AccessStr = JsonObject->GetStringField(TEXT("AccessMode"));
		UEnum* AccessEnum = StaticEnum<EMassFragmentAccess>();
		const int64 AccessVal = AccessEnum->GetValueByNameString(AccessStr);
		if (AccessVal == INDEX_NONE)
		{
			UE_LOGF(LogMass, Warning, "Mass Debugger: cannot resolve AccessMode '%ls' for type '%ls' - entry skipped.", *AccessStr, *StructPath);
			return false;
		}

		const FString PresenceStr = JsonObject->GetStringField(TEXT("Presence"));
		UEnum* PresenceEnum = StaticEnum<EMassFragmentPresence>();
		const int64 PresenceVal = PresenceEnum->GetValueByNameString(PresenceStr);
		if (PresenceVal == INDEX_NONE)
		{
			UE_LOGF(LogMass, Warning, "Mass Debugger: cannot resolve Presence '%ls' for type '%ls' - entry skipped.", *PresenceStr, *StructPath);
			return false;
		}

		StructType = Found;
		AccessMode = (EMassFragmentAccess)AccessVal;
		Presence = (EMassFragmentPresence)PresenceVal;

		return true;
	}

	bool FFragmentEntry::DeserializeFromJsonString(const FString& JsonString)
	{
		TSharedPtr<FJsonObject> JsonObject;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(JsonString);

		if (!FJsonSerializer::Deserialize(Reader, JsonObject) || !JsonObject.IsValid())
		{
			return false;
		}

		return DeserializeFromJson(JsonObject);
	}

	TSharedPtr<FJsonObject> FEditableQuery::SerializeToJson() const
	{
		TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
		JsonObject->SetStringField(TEXT("Name"), Name);

		TArray<TSharedPtr<FJsonValue>> JsonFragments;
		JsonFragments.Reserve(FragmentRequirements.Num());
		for (const TSharedPtr<FFragmentEntry>& Entry : FragmentRequirements)
		{
			if (!Entry.IsValid())
			{
				continue;
			}
			if (TSharedPtr<FJsonObject> EntryJson = Entry->SerializeToJson())
			{
				JsonFragments.Add(MakeShared<FJsonValueObject>(MoveTemp(EntryJson)));
			}
		}
		JsonObject->SetArrayField(TEXT("FragmentRequirements"), MoveTemp(JsonFragments));

		TArray<TSharedPtr<FJsonValue>> JsonTags;
		JsonTags.Reserve(TagRequirements.Num());
		for (const TSharedPtr<FFragmentEntry>& Entry : TagRequirements)
		{
			if (!Entry.IsValid())
			{
				continue;
			}
			if (TSharedPtr<FJsonObject> EntryJson = Entry->SerializeToJson())
			{
				JsonTags.Add(MakeShared<FJsonValueObject>(MoveTemp(EntryJson)));
			}
		}
		JsonObject->SetArrayField(TEXT("TagRequirements"), MoveTemp(JsonTags));

		return JsonObject;
	}

	bool FEditableQuery::DeserializeFromJson(const TSharedPtr<FJsonObject>& JsonObject)
	{
		if (!JsonObject.IsValid())
		{
			return false;
		}

		if (JsonObject->HasField(TEXT("Name")))
		{
			Name = JsonObject->GetStringField(TEXT("Name"));
		}

		FragmentRequirements.Reset();
		const TArray<TSharedPtr<FJsonValue>>* JsonFragments = nullptr;
		if (JsonObject->TryGetArrayField(TEXT("FragmentRequirements"), JsonFragments))
		{
			for (const TSharedPtr<FJsonValue>& Val : *JsonFragments)
			{
				if (TSharedPtr<FJsonObject> Obj = Val->AsObject())
				{
					TSharedPtr<FFragmentEntry> Entry = MakeShared<FFragmentEntry>();
					if (!Entry->DeserializeFromJson(Obj))
					{
						// Skip individual unresolvable entries so the rest of the query still loads.
						continue;
					}
					Entry->bShowAccessMode = true;
					FragmentRequirements.Add(MoveTemp(Entry));
				}
			}
		}

		TagRequirements.Reset();
		const TArray<TSharedPtr<FJsonValue>>* JsonTags = nullptr;
		if (JsonObject->TryGetArrayField(TEXT("TagRequirements"), JsonTags))
		{
			for (const TSharedPtr<FJsonValue>& Val : *JsonTags)
			{
				if (TSharedPtr<FJsonObject> Obj = Val->AsObject())
				{
					TSharedPtr<FFragmentEntry> Entry = MakeShared<FFragmentEntry>();
					if (!Entry->DeserializeFromJson(Obj))
					{
						continue;
					}
					Entry->bShowAccessMode = false;
					TagRequirements.Add(MoveTemp(Entry));
				}
			}
		}

		return true;
	}

	bool FEditableQuery::DeserializeFromJsonString(const FString& JsonString)
	{
		TSharedPtr<FJsonObject> JsonObject;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(JsonString);

		if (FJsonSerializer::Deserialize(Reader, JsonObject) && JsonObject.IsValid())
		{
			return DeserializeFromJson(JsonObject);
		}

		return false;
	}
} // namespace UE::MassDebugger


#undef LOCTEXT_NAMESPACE
