// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MassArchetypeTypes.h"
#include "MassEntityQuery.h"
#include "MassProcessorDependencySolver.h"
#include "MassProcessingTypes.h"
#include "MassProcessor.h"
#include "Types/SlateEnums.h"
#include "MassDebugger.h"


class FJsonObject;
class UMassProcessor;
struct FMassArchetypeHandle;
struct FMassDebuggerModel;
struct FMassEntityHandle;
struct FMassEntityManager;
struct FMassEntityQuery;
class UWorld;
class SMassEntitiesView;
class SMassDebugger;

namespace UE::MassDebugger
{
	class SQueryEditorView;

	/** A serializable wrapper for a fragment type and access method used for editable queries */
	struct FFragmentEntry
	{
		/** Convert this struct to a Json object */
		TSharedPtr<FJsonObject> SerializeToJson() const;

		/** Populate this struct from a Json object created with the above Serialize method */
		bool DeserializeFromJson(const TSharedPtr<FJsonObject>& JsonObject);

		/** Parse Json string then serialize from that (paste) */
		bool DeserializeFromJsonString(const FString& JsonString);

		/** The current struct type (selected in the editor combo box) */
		TWeakObjectPtr<const UScriptStruct> StructType;

		/** Access mode for this fragment entry in a query (not used for tags) */
		EMassFragmentAccess AccessMode = EMassFragmentAccess::ReadOnly;

		/** Presense requred for this fragment in a query */
		EMassFragmentPresence Presence = EMassFragmentPresence::All;

		/** Used to hide access mode when unused (e.g. with tags) */
		bool bShowAccessMode = true;
	};

	/** A serializable and editable representation of a FMassEntityQuery */
	struct FEditableQuery
	{
		/**
		 * Convert this entire object and it FragmentEntry arrays to Json
		 * Used to save user queries created in the debugger UI
		 */
		TSharedPtr<FJsonObject> SerializeToJson() const;

		/** 
		 * Deserialize Json object hierarchy and populate this struct
		 * Used to load saved user queries when opening the debugger
		 */
		bool DeserializeFromJson(const TSharedPtr<FJsonObject>& JsonObject);

		/**
		 * Parse the Json string then deserialize
		 * Used to paste queries in the debugger
		 */
		bool DeserializeFromJsonString(const FString& JsonString);

		/**
		 * Create an editable query from a runtime query struct
		 * Used to view/edit and use runtime queries (e.g. on a Processor) in the editor
		 */
		void InitFromEntityQuery(const FMassEntityQuery& InQuery, FMassDebuggerModel& DebuggerModel);

		/** The visible/editable name of the query */
		FString Name;

		/** Editable representation of the fragment requirements for the query */
		TArray<TSharedPtr<FFragmentEntry>> FragmentRequirements;

		/** Editable representation of the tag requirements for the query */
		TArray<TSharedPtr<FFragmentEntry>> TagRequirements;

		/**
		 * Create a usable runtime query from this editable query
		 * Used by the debugger to manually run entity queries or for breakpoints
		 */
		FMassEntityQuery BuildEntityQuery(const TSharedRef<FMassEntityManager> EntityManager);
	};
} // namespace UE::MassDebugger

enum class EMassDebuggerSelectionMode : uint8
{
	None,
	Processor,
	Archetype,
	// @todo future:
	// Fragment
	MAX	
};

enum class EMassDebuggerProcessorSelection : uint8
{
	None,
	Selected,
	MAX
};

enum class EMassDebuggerProcessingGraphNodeSelection : uint8
{
	None,
	WaitFor,
	Block,
	MAX
};


struct FMassDebuggerQueryData
{
	FMassDebuggerQueryData(const FMassEntityQuery& Query, const FText& InLabel);
	FMassDebuggerQueryData(const FMassSubsystemRequirements& SubsystemRequirements, const FText& InLabel);

	FMassExecutionRequirements ExecutionRequirements;
	FText Label;
	FText AdditionalInformation;

	FMassEntityQuery SourceQuery;

	int32 GetTotalBitsUsedCount();
	bool IsEmpty() const;
}; 

struct FMassDebuggerArchetypeData
{
	FMassDebuggerArchetypeData(const FMassArchetypeHandle& ArchetypeHandle);

	FMassArchetypeHandle Handle;

	FMassArchetypeCompositionDescriptor Composition;

	FMassElementBitSet SparseElementsBitset;

	/** Hash of the Compositions. */
	uint32 CompositionHash = 0;
	/** Combined hash of composition and shared fragments. */
	uint32 FullHash = 0;

	/** Archetype statistics */
	UE::Mass::Debug::FArchetypeStats ArchetypeStats;

	/** Child debugger data (same as parent, but changed in some way) */
	TArray<TSharedPtr<FMassDebuggerArchetypeData>> Children;
	/** Parent debugger data. */
	TWeakPtr<FMassDebuggerArchetypeData> Parent;


	/** Index in FMassDebuggerModel::CachedArchetypes */
	int32 Index = INDEX_NONE;
	/** Display label */
	FText Label;
	/** Display label */
	FText LabelLong;
	/** Display label tooltip */
	FText LabelTooltip;
	/** FullHash as a display string */
	FText HashLabel;
	/** Primary debug name, used for grouping derived archetypes. */
	FString PrimaryDebugName;

	/** True if the archetype is selected. */
	bool bIsSelected = false;

	int32 GetTotalBitsUsedCount() const;
};

struct FMassDebuggerProcessorData
{
	FMassDebuggerProcessorData(const UMassProcessor& InProcessor);
	FMassDebuggerProcessorData(const FMassEntityManager& InEntityManager, const UMassProcessor& InProcessor, const TMap<FMassArchetypeHandle, TSharedPtr<FMassDebuggerArchetypeData>>& InTransientArchetypesMap);

private:
	void SetProcessor(const UMassProcessor& InProcessor);

public:
	FString Name;
	FString Label;
	uint32 ProcessorHash = 0;
	bool bIsActive = true;
	TWeakPtr<const FMassEntityManager> EntityManager;
	TWeakObjectPtr<const UMassProcessor> Processor;
	EMassDebuggerProcessorSelection Selection = EMassDebuggerProcessorSelection::None;
	TSharedPtr<FMassDebuggerQueryData> ProcessorRequirements;
	TArray<TSharedPtr<FMassDebuggerQueryData>> Queries;
	TArray<TSharedPtr<FMassDebuggerArchetypeData>> ValidArchetypes;
#if WITH_MASSENTITY_DEBUG
	FString Description;
#endif // WITH_MASSENTITY_DEBUG	
};

struct FMassDebuggerFragmentData
{
	FMassDebuggerFragmentData(TNotNull<const UScriptStruct*> InFragment);

	bool operator==(const FMassDebuggerFragmentData& Other) const
	{
		return Fragment == Other.Fragment;
	}

	friend uint32 GetTypeHash(const FMassDebuggerFragmentData& Item)
	{
		return GetTypeHash(Item.Fragment);
	}

	FText Name;
	TWeakObjectPtr<const UScriptStruct> Fragment;
	TWeakPtr<const FMassEntityManager> EntityManager;
	TArray<FMassArchetypeHandle> Archetypes;
	TArray<TWeakObjectPtr<UMassProcessor>> Processors;
	int32 NumEntities = 0;
};


struct FMassDebuggerBreakpointData
{
	FMassDebuggerBreakpointData() = default;
	FMassDebuggerBreakpointData(const UE::Mass::Debug::FBreakpoint& InBreakpoint, FMassDebuggerModel& Model);
	void ReconcileDataFromNames(FMassDebuggerModel& Model);
	void ReconcileDataFromEngineBreakpoint(FMassDebuggerModel& Model);
	void ApplyToEngine(FMassDebuggerModel& Model, bool bRefreshEngineBreakpoints = true);

	TSharedPtr<FJsonObject> SerializeToJson() const;
	bool DeserializeFromJson(const TSharedPtr<FJsonObject>& JsonObject);
	bool DeserializeFromJsonString(const FString& JsonString);

	UE::Mass::Debug::FBreakpoint BreakpointInstance;
	FString TriggerName;
	FString FilterName;
	TSharedPtr<FMassDebuggerFragmentData> TriggerFragment;
	TSharedPtr<FMassDebuggerProcessorData> TriggerProcessor;
	TSharedPtr<UE::MassDebugger::FEditableQuery> FilterQuery;
};

struct FMassDebuggerProcessingGraphNode
{
	explicit FMassDebuggerProcessingGraphNode(const TSharedPtr<FMassDebuggerProcessorData>& InProcessorData, const UMassCompositeProcessor::FDependencyNode& InProcessorNode = UMassCompositeProcessor::FDependencyNode());
	
	FText GetLabel() const;

	TSharedPtr<FMassDebuggerProcessorData> ProcessorData;
	TArray<int32> WaitForNodes;
	TArray<int32> BlockNodes;
	EMassDebuggerProcessingGraphNodeSelection GraphNodeSelection = EMassDebuggerProcessingGraphNodeSelection::None;
};

struct FMassDebuggerProcessingGraph
{
	FMassDebuggerProcessingGraph(const FMassDebuggerModel& DebuggerModel, TNotNull<const UMassCompositeProcessor*> InGraphOwner);

	FString Label;
	TArray<FMassDebuggerProcessingGraphNode> GraphNodes;
	bool bSingleTheadGraph = !bool(MASS_DO_PARALLEL);
};


struct FMassDebuggerEnvironment
{
	explicit FMassDebuggerEnvironment(const TSharedRef<const FMassEntityManager>& InEntityManager);

	bool operator==(const FMassDebuggerEnvironment& Other) const { return EntityManager == Other.EntityManager; }

	FString GetDisplayName() const;
	TSharedPtr<const FMassEntityManager> GetEntityManager() const;
	TSharedPtr<FMassEntityManager> GetMutableEntityManager() const;
	bool IsWorldValid() const { return World.IsValid(); }
	bool NeedsValidWorld() const { return bNeedsValidWorld; }
	
	TWeakPtr<const FMassEntityManager> EntityManager;
	TMap<FName, UE::Mass::Debug::FProcessorProviderFunction> ProcessorProviders;
	TWeakObjectPtr<UWorld> World;
	bool bNeedsValidWorld = false;
};


struct FMassDebuggerModel
{
	DECLARE_MULTICAST_DELEGATE(FOnRefresh);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnProcessorsSelected, TConstArrayView<TSharedPtr<FMassDebuggerProcessorData>>, ESelectInfo::Type);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnArchetypesSelected, TConstArrayView<TSharedPtr<FMassDebuggerArchetypeData>>, ESelectInfo::Type);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnFragmentSelected, FName FragmentName);
	DECLARE_MULTICAST_DELEGATE(FOnQueriesChanged);
	DECLARE_MULTICAST_DELEGATE(FOnBreakpointsChanged);

	FMassDebuggerModel();
	~FMassDebuggerModel();

	void SetEnvironment(const TSharedPtr<FMassDebuggerEnvironment>& Item);

	void RefreshAll();

	void LoadQueriesFromDisk();

	void SelectProcessor(TSharedPtr<FMassDebuggerProcessorData>& Processor);
	void SelectProcessors(TArrayView<TSharedPtr<FMassDebuggerProcessorData>> Processors, ESelectInfo::Type SelectInfo);
	void ClearProcessorSelection();

	void SelectArchetypes(TArrayView<TSharedPtr<FMassDebuggerArchetypeData>> SelectedArchetypes, ESelectInfo::Type SelectInfo);
	void ClearArchetypeSelection();

	bool IsCurrentEnvironment(const FMassDebuggerEnvironment& InEnvironment) const { return Environment && *Environment.Get() == InEnvironment; }
	bool IsCurrentEnvironmentValid() const { return Environment && Environment->EntityManager.IsValid(); }
	bool HasEnvironmentSelected() const { return static_cast<bool>(Environment); }

	void CacheArchetypesData(TMap<FMassArchetypeHandle, TSharedPtr<FMassDebuggerArchetypeData>>& OutTransientArchetypesMap); 
	void CacheProcessorsData(const TMap<FMassArchetypeHandle, TSharedPtr<FMassDebuggerArchetypeData>>& InTransientArchetypesMap, TArray<TNotNull<const UMassCompositeProcessor*>>& OutCompositeProcessors);
	void CacheProcessingGraphs(TConstArrayView<TNotNull<const UMassCompositeProcessor*>> InCompositeProcessors);

	void CacheFragmentData(
		TArray<TSharedPtr<FMassDebuggerFragmentData>>& OutData,
		const TConstArrayView<TWeakObjectPtr<const UScriptStruct>>& FragmentTypes,
		bool bAppend
	);
	void CacheFragmentData();
	TSharedPtr<FMassDebuggerFragmentData> FindFragmentData(const UScriptStruct* FragmentType);

	void CacheTagData();
	
	void LoadBreakpointsFromDisk();
	void SaveBreakpointsToDisk();
	void CreateBreakpoint();
	void CreateBreakpointFromString(const FString& InString);
	void ApplyBreakpointsToCurrentEnvironment();
	void ReconcileAllBreakpoints();
	void RemoveAllBreakpoints();
	void RemoveBreakpoint(UE::Mass::Debug::FBreakpointHandle Handle);

	float MinDistanceToSelectedArchetypes(const TSharedPtr<FMassDebuggerArchetypeData>& InArchetypeData) const;

	FText GetDisplayName() const;

	void MarkAsStale();
	bool IsStale() const;

	const TSharedPtr<FMassDebuggerProcessorData>& GetProcessorDataChecked(const UMassProcessor& Processor) const;

	void RegisterEntitiesView(TSharedRef<SMassEntitiesView> EntitiesView, int32 Index);

	static const int32 MaxEntityViewCount = 1;
	void ShowEntitiesView(int Index, FMassArchetypeHandle ArchetypeHandle);
	void ShowEntitiesView(int Index, TArray<FMassEntityHandle> EntitieHandles);
	void ShowEntitiesView(int Index, FMassEntityQuery& Query);
	void ShowEntitiesView(int Index, TConstArrayView<FMassEntityQuery*> InQueries);
	void ResetEntitiesViews();
	TWeakPtr<SMassEntitiesView> ShowEntitiesView(int32 Index);

	void RegisterQueryEditor(TSharedRef<UE::MassDebugger::SQueryEditorView> InQueryEditorView);
	void ShowQueryInEditor(const FMassEntityQuery& InQuery, const FString& InQueryName);
	void RefreshQueries();

protected:
	void StoreArchetypes(const FMassEntityManager& EntityManager, TMap<FMassArchetypeHandle, TSharedPtr<FMassDebuggerArchetypeData>>& OutTransientArchetypesMap);

	void ResetSelectedArchetypes();
	void ResetSelectedProcessors();

	void OnEntitySelected(const FMassEntityManager& EntityManager, const FMassEntityHandle EntityHandle);
	void OnBreakpointsChanged();

public:
	TWeakPtr<SMassDebugger> DebuggerWindow;

	FOnRefresh OnRefreshDelegate;
	FOnProcessorsSelected OnProcessorsSelectedDelegate;
	FOnArchetypesSelected OnArchetypesSelectedDelegate;
	FOnFragmentSelected OnFragmentSelectedDelegate;
	FOnBreakpointsChanged OnEditorBreakpointsChangedDelegate;
	FOnQueriesChanged OnQueriesChangedDelegate;

	EMassDebuggerSelectionMode SelectionMode = EMassDebuggerSelectionMode::None;

	TSharedPtr<FMassDebuggerEnvironment> Environment;
	struct FProcessorCollection
	{
		FProcessorCollection(FName InLabel = FName())
			: Label(InLabel)
		{	
		}
		FName Label;
		TArray<TSharedPtr<FMassDebuggerProcessorData>> Container;
	};

	TArray<TSharedPtr<UE::MassDebugger::FEditableQuery>> QueryList;
	TArray<TSharedPtr<FMassDebuggerFragmentData>> CachedTagData;
	TArray<TSharedPtr<FMassDebuggerFragmentData>> CachedFragmentData;
	TArray<TSharedPtr<FProcessorCollection>> CachedProcessorCollections;
	TArray<TSharedPtr<FMassDebuggerProcessorData>> SelectedProcessors;
	TArray<TSharedPtr<FMassDebuggerArchetypeData>> CachedAllArchetypes;
	TArray<TSharedPtr<FMassDebuggerArchetypeData>> CachedArchetypeRepresentatives;
	TArray<TSharedPtr<FMassDebuggerArchetypeData>> SelectedArchetypes;
	TArray<TSharedPtr<FMassDebuggerProcessingGraph>> CachedProcessingGraphs;
	TArray<TSharedPtr<FMassDebuggerBreakpointData>> CachedBreakpoints;

	/** Name lists and maps for easy lookup from widgets */
	TArray<TSharedPtr<FString>> ProcessorNames;
	TMap<FString, TWeakPtr<FMassDebuggerProcessorData>> ProcessorMap;
	TArray<TSharedPtr<FString>> FragmentNames;
	TMap<FString, TWeakPtr<FMassDebuggerFragmentData>> FragmentMap;
	TArray<TSharedPtr<FString>> TagNames;
	TMap<FString, TWeakPtr<FMassDebuggerFragmentData>> TagMap;
	TArray<TSharedPtr<FString>> QueryNames;
	TMap<FString, TWeakPtr<UE::MassDebugger::FEditableQuery>> QueryMap;

	TMap<FMassArchetypeHandle, TSharedPtr<FMassDebuggerArchetypeData>> HandleToArchetypeMap;

	TArray<TArray<float>> ArchetypeDistances;

	FString EnvironmentDisplayName;

	FDelegateHandle OnEntitySelectedHandle;
	FDelegateHandle OnBreakpointsChangedHandle;

	void SelectFragment(FName InFragementName);
	FName GetSelectedFragment();

protected:	
	TArray<TSharedPtr<FMassDebuggerProcessorData>> AllCachedProcessors;
	TArray<TWeakPtr<SMassEntitiesView>> EntityViews;
	TWeakPtr<UE::MassDebugger::SQueryEditorView> QueryEditorView;
	FName SelectedFragmentName;

public:
	UE_DEPRECATED(5.6, "CachedProcessors property is now deprecated. Use CachedProcessorCollections instead. ")
	TArray<TSharedPtr<FMassDebuggerProcessorData>> CachedProcessors;
};

