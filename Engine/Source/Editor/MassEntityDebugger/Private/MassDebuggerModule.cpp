// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassProcessingTypes.h"
#include "IMassDebuggerModule.h"
#include "Features/IModularFeatures.h"

#if WITH_MASSENTITY_DEBUG

#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#include "SMassDebugger.h"
#include "MassDebuggerStyle.h"

#define LOCTEXT_NAMESPACE "Mass"

static const FName MassDebugTabName("MassDebugger");

class FMassDebuggerModule : public IMassDebuggerModule, public IModularFeature
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;

private:
	TSharedRef<SDockTab> SpawnTab(const FSpawnTabArgs& SpawnTabArgs);
};

IMPLEMENT_MODULE(FMassDebuggerModule, MassEntityDebugger)


void FMassDebuggerModule::StartupModule()
{
	// This code will execute after your module is loaded into memory (but after global variables are initialized, of course.)
	FMassDebuggerStyle::Initialize();
	FMassDebuggerCommands::Register();

	// register the UI tab
	IModularFeatures::Get().RegisterModularFeature(MassDebugTabName, this);

	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(
		MassDebugTabName,
		FOnSpawnTab::CreateRaw(this, &FMassDebuggerModule::SpawnTab))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsDebugCategory())
		.SetDisplayName(NSLOCTEXT("MassDebuggerApp", "TabTitle", "Mass Debugger"))
		.SetTooltipText(NSLOCTEXT("MassDebuggerApp", "TooltipText", "Opens Mass Debugger tool."))
		.SetIcon(FSlateIcon(FMassDebuggerStyle::GetStyleSetName(), "MassDebuggerApp.TabIcon"));
}

TSharedRef<SDockTab> FMassDebuggerModule::SpawnTab(const FSpawnTabArgs& SpawnTabArgs)
{
	const TSharedRef<SDockTab> MajorTab = SNew(SMassDebuggerTab)
		.TabRole(ETabRole::NomadTab);

	TSharedPtr<SWidget> TabContent;

	TabContent = SNew(SMassDebugger, MajorTab, SpawnTabArgs.GetOwnerWindow());

	MajorTab->SetContent(TabContent.ToSharedRef());

	return MajorTab;
}

namespace UE::Mass::Debug::Private
{
	FAutoConsoleCommand SummonDebbugerCommand(
		TEXT("mass.debug"),
		TEXT("Summon Mass Debugger UI"),
		FConsoleCommandDelegate::CreateLambda([]() 
		{ 
			FGlobalTabmanager::Get()->TryInvokeTab(MassDebugTabName);
		})
	);

} // UE::Mass::Debug::Private

#else // WITH_MASSENTITY_DEBUG

class FMassDebuggerModule : public IMassDebuggerModule, public IModularFeature
{
};

IMPLEMENT_MODULE(FMassDebuggerModule, MassEntityDebugger)

#endif // WITH_MASSENTITY_DEBUG

#undef LOCTEXT_NAMESPACE
