// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassDebuggerStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/AppStyle.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/Font.h"
#include "Components/Widget.h"
#include "Styling/StyleColors.h"
#include "Textures/SlateIcon.h"

class FMassDebuggerStyleSet final : public FSlateStyleSet
{
public:
	static FString InContent(const FString& RelativePath, const ANSICHAR* Extension)
	{
		static FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("MassEntity"))->GetContentDir() / TEXT("Slate");
		return (ContentDir / RelativePath) + Extension;
	}

	FMassDebuggerStyleSet(const FName& InStyleSetName)
		: FSlateStyleSet(InStyleSetName)
	{
		const FVector2f Icon16x16(16.0f, 16.0f);

		SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
		SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

		Set("MassDebuggerApp.TabIcon", new IMAGE_BRUSH_SVG("Starship/Common/Debug", Icon16x16));

		const FTextBlockStyle& NormalText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");

		const FSlateFontInfo LargeFont(DEFAULT_FONT("Bold", 50));

		FTextBlockStyle StateTitle = FTextBlockStyle(NormalText)
			.SetFont(LargeFont)
			.SetColorAndOpacity(FLinearColor(230.0f / 255.0f, 230.0f / 255.0f, 230.0f / 255.0f, 0.9f));
		Set("MassDebug.Font.Large", StateTitle);

		Set("MassDebug.Fragment", new CORE_BOX_BRUSH("Common/LightGroupBorder", FMargin(4.0f / 16.0f), FLinearColor(.2f, .2f, .9f)));
		Set("MassDebug.Fragment.Added", new CORE_BOX_BRUSH("Common/LightGroupBorder", FMargin(4.0f / 16.0f), FLinearColor(.2f, .2f, .9f)));
		Set("MassDebug.Fragment.Removed", new CORE_BOX_BRUSH("Common/LightGroupBorder", FMargin(4.0f / 16.0f), FLinearColor(.9f, .2f, .2f)));
		Set("MassDebug.Fragment.ReadOnly", new CORE_BOX_BRUSH("Common/LightGroupBorder", FMargin(4.0f / 16.0f), FLinearColor(.5f, .5f, .5f)));
		Set("MassDebug.Fragment.ReadWrite", new CORE_BOX_BRUSH("Common/LightGroupBorder", FMargin(4.0f / 16.0f), FLinearColor(0.f, .7f, 0.f)));

		Set("MassDebug.Processor", new FSlateRoundedBoxBrush(FLinearColor::Gray, 10.0f));
		Set("MassDebug.Processor.AccessRequired", new FSlateRoundedBoxBrush(FLinearColor::Blue, 10.0f));
		Set("MassDebug.Processor.AccessRead", new FSlateRoundedBoxBrush(FLinearColor::Green, 10.0f));
		Set("MassDebug.Processor.AccessWrite", new FSlateRoundedBoxBrush(FLinearColor::Red, 10.0f));
		Set("MassDebug.Processor.AccessBlock", new FSlateRoundedBoxBrush(FLinearColor::Gray * 0.25f, 10.0f));

		Set("MassDebug.Processor.InnerBackground", new FSlateRoundedBoxBrush(FLinearColor::Black.CopyWithNewOpacity(0.75f), 10.0f));

		Set("MassDebug.Label.Background", new FSlateRoundedBoxBrush(FStyleColors::Foreground, 4.0f));

		Set("MassDebug.Label.Text", FTextBlockStyle(NormalText)
			.SetFont(FSlateFontInfo(DEFAULT_FONT("Bold", 7)))
			.SetColorAndOpacity(FStyleColors::Background));
	}
};


TSharedPtr<FSlateStyleSet> FMassDebuggerStyle::StyleSet = nullptr;


void FMassDebuggerStyle::Initialize()
{
	if (StyleSet.IsValid())
	{
		return;
	}

	StyleSet = MakeShared<FMassDebuggerStyleSet>(GetStyleSetName());
	
	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
}

void FMassDebuggerStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}
}

FName FMassDebuggerStyle::GetStyleSetName()
{
	static FName StyleName("MassDebuggerStyle");
	return StyleName;
}
