// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

#define UE_API MASSENTITYDEBUGGER_API


class FMassDebuggerStyle
{
public:
	static UE_API void Initialize();
	static UE_API void Shutdown();
	static ISlateStyle& Get() { return *StyleSet.Get(); }
	static UE_API FName GetStyleSetName();

	static const FSlateBrush* GetBrush(FName PropertyName, const ANSICHAR* Specifier = NULL)
	{
		return StyleSet->GetBrush(PropertyName, Specifier);
	}

private:
	static UE_API TSharedPtr<FSlateStyleSet> StyleSet;
};

#undef UE_API
