// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMassArchetype.h"
#include "MassDebuggerModel.h"
#include "MassEntityTypes.h"
#include "SMassBitSet.h"
#include "SMassEntitiesList.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SMassDebugger"

//----------------------------------------------------------------------//
// SMassArchetype
//----------------------------------------------------------------------//
void SMassArchetype::Construct(const FArguments& InArgs, TSharedPtr<FMassDebuggerArchetypeData> InArchetypeData, TSharedPtr<FMassDebuggerArchetypeData> InBaseArchetypeData
	, const EMassBitSetDiffPrune Prune, TSharedRef<FMassDebuggerModel> InDebuggerModel)
{
	if (!InArchetypeData)
	{
		return;
	}

	ArchetypeData = InArchetypeData;
	DebuggerModel = InDebuggerModel;

	const FMassDebuggerArchetypeData* BaseArchetypeDebugData = InBaseArchetypeData.Get();
	const FMassDebuggerArchetypeData& ArchetypeDebugData = *InArchetypeData.Get();

	if (BaseArchetypeDebugData == &ArchetypeDebugData)
	{
		BaseArchetypeDebugData = nullptr;
	}

	TSharedRef<SVerticalBox> Box = SNew(SVerticalBox);
	const TArray<FText> LabelBits = {
		LOCTEXT("MassArchetypeLabel", "Archetype")
		, InArchetypeData->LabelLong
	};

	Box->AddSlot()
		.AutoHeight()
		.Padding(0, 4)
		[
			SNew(SButton)
				.Text(LOCTEXT("ShowEntities", "Show Entities"))
				.ContentPadding(4)
				.OnClicked(FOnClicked::CreateSP(this, &SMassArchetype::ShowEntities))
		];

	Box->AddSlot()
		.AutoHeight()
		.Padding(0, 4)
		[
			SNew(STextBlock)
				.Text(InArchetypeData->HashLabel)
				.Font(FCoreStyle::GetDefaultFontStyle("Mono", 9))
		];

	FText ArchetypeDescription = FText::Format(LOCTEXT("ArchetypeDescription", "EntitiesCount: {0}\
		\nBytesPerEntity: {1}\
		\nEntitiesCountPerChunk: {2}\
		\nChunksCount: {3}\
		\nAllocated memory: {4}\
		\nWasted memory : {5} ({6}%)\
		\nSparseElements bookkeeping: {7}")
		, FText::AsNumber(ArchetypeDebugData.ArchetypeStats.EntitiesCount)
		, FText::AsMemory(ArchetypeDebugData.ArchetypeStats.BytesPerEntity)
		, FText::AsNumber(ArchetypeDebugData.ArchetypeStats.EntitiesCountPerChunk)
		, FText::AsNumber(ArchetypeDebugData.ArchetypeStats.ChunksCount)
		, FText::AsMemory(ArchetypeDebugData.ArchetypeStats.AllocatedSize)
		, FText::AsMemory(ArchetypeDebugData.ArchetypeStats.WastedEntityMemory)
		, FText::AsNumber((ArchetypeDebugData.ArchetypeStats.AllocatedSize != 0
			? static_cast<float>(ArchetypeDebugData.ArchetypeStats.WastedEntityMemory) * 100.f / ArchetypeDebugData.ArchetypeStats.AllocatedSize
			: 0.f))
		, FText::AsMemory(ArchetypeDebugData.ArchetypeStats.SparseElementsBookkeepingSize)
	);

	Box->AddSlot()
		.AutoHeight()
		.Padding(0, 4)
		[
			SNew(STextBlock)
			.Text(ArchetypeDescription)
		];

	if (ArchetypeDebugData.ArchetypeStats.EntitiesCountPerChunk != 0 && ArchetypeDebugData.ArchetypeStats.ChunksCount != 0)
	{
		const float AverageEntitiesPerChunkActual = float(ArchetypeDebugData.ArchetypeStats.EntitiesCount) / ArchetypeDebugData.ArchetypeStats.ChunksCount;
		FText DerivedArchetypeDescription = FText::Format(LOCTEXT("ArchetypeDescriptionAux", "Actual average Entities per Chunk: {0}\nChunk occupancy: {1}")
			, FText::AsNumber(AverageEntitiesPerChunkActual)
			, FText::AsNumber(AverageEntitiesPerChunkActual / ArchetypeDebugData.ArchetypeStats.EntitiesCountPerChunk));

		Box->AddSlot()
			.AutoHeight()
			.Padding(0, 4)
			[
				SNew(STextBlock)
					.Text(DerivedArchetypeDescription)
			];
	}

	const FMassArchetypeCompositionDescriptor& Composition = ArchetypeData->Composition;
	const FSlateBrush* Brush = FMassDebuggerStyle::GetBrush("MassDebug.Fragment");

	if (BaseArchetypeDebugData != nullptr)
	{
		const FMassArchetypeCompositionDescriptor& ParentComposition = BaseArchetypeDebugData->Composition;

		UE::Mass::Debugger::UI::AddBitSetDiff(Box, ParentComposition.DebugGetFragments(), Composition.DebugGetFragments(), TEXT("Fragments"), Brush, Prune, InDebuggerModel);
		UE::Mass::Debugger::UI::AddBitSetDiff(Box, ParentComposition.DebugGetTags(), Composition.DebugGetTags(), TEXT("Tags"), Brush, Prune, InDebuggerModel);
		UE::Mass::Debugger::UI::AddBitSetDiff(Box, ParentComposition.DebugGetChunkFragments(), Composition.DebugGetChunkFragments(), TEXT("Chunk Fragments"), Brush, Prune, InDebuggerModel);
		UE::Mass::Debugger::UI::AddBitSetDiff(Box, ParentComposition.DebugGetSharedFragments(), Composition.DebugGetSharedFragments(), TEXT("Shared Fragments"), Brush, Prune, InDebuggerModel);
		UE::Mass::Debugger::UI::AddBitSetDiff(Box, ParentComposition.DebugGetConstSharedFragments(), Composition.DebugGetConstSharedFragments(), TEXT("Const Shared Fragments"), Brush, Prune, InDebuggerModel);
	}
	else
	{
		UE::Mass::Debugger::UI::AddBitSet(Box, Composition.DebugGetFragments(), TEXT("Fragments"), Brush, InDebuggerModel);
		UE::Mass::Debugger::UI::AddBitSet(Box, Composition.DebugGetTags(), TEXT("Tags"), Brush, InDebuggerModel);
		UE::Mass::Debugger::UI::AddBitSet(Box, Composition.DebugGetChunkFragments(), TEXT("Chunk Fragments"), Brush, InDebuggerModel);
		UE::Mass::Debugger::UI::AddBitSet(Box, Composition.DebugGetSharedFragments(), TEXT("Shared Fragments"), Brush, InDebuggerModel);
		UE::Mass::Debugger::UI::AddBitSet(Box, Composition.DebugGetConstSharedFragments(), TEXT("Const Shared Fragments"), Brush, InDebuggerModel);
	}

	UE::Mass::Debugger::UI::AddBitSet(Box, ArchetypeData->SparseElementsBitset, TEXT("Sparse Elements"), Brush, InDebuggerModel);

	TSharedRef<SVerticalBox> MainBox = SNew(SVerticalBox);

	MainBox->AddSlot()
	.AutoHeight()
	.Padding(0, 4)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			SNew(SRichTextBlock)
			.Text(FText::Join(FText::FromString(TEXT(": ")), LabelBits))
			.DecoratorStyleSet(&FAppStyle::Get())
			.TextStyle(FAppStyle::Get(), "LargeText")
		]
	];
	MainBox->AddSlot()
	.AutoHeight()
	.Padding(0, 4)
	[
		Box
	];

	ChildSlot
	[
		SNew(SBorder)
		.Padding(5.0f)
		[
			MainBox
		]
	];
}

FReply SMassArchetype::ShowEntities()
{
	if (DebuggerModel.IsValid() && ArchetypeData.IsValid())
	{
		DebuggerModel->ShowEntitiesView(0, ArchetypeData->Handle);
	}
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE