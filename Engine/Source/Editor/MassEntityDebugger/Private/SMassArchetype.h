// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/SCompoundWidget.h"
#include "SMassBitSet.h"

struct FMassDebuggerArchetypeData;
struct FMassEntityHandle;
struct FMassDebuggerModel;

class SMassArchetype : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMassArchetype){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FMassDebuggerArchetypeData> InArchetypeData, TSharedPtr<FMassDebuggerArchetypeData> InBaseArchetypeData, const EMassBitSetDiffPrune Prune, TSharedRef<FMassDebuggerModel> InDebuggerModel);

protected:
	TSharedPtr<FMassDebuggerArchetypeData> ArchetypeData;
	TSharedPtr<FMassDebuggerModel> DebuggerModel;
	bool bBitSetsVisible = true;
	bool bEntitiesVisible = true;
	FReply ShowEntities();
};
