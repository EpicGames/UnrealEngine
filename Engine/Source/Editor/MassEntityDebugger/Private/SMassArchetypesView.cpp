// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMassArchetypesView.h"
#include "MassDebuggerModel.h"
#include "SMassArchetype.h"
#include "MassDebuggerStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Text/SRichTextBlock.h"

#define LOCTEXT_NAMESPACE "SMassDebugger"


namespace UE::Mass::Debugger::UI::Private
{
	FLinearColor GetArchetypeDistanceColor(float Distance)
	{
		const FLinearColor Foreground = FStyleColors::Foreground.GetSpecifiedColor();
		const FLinearColor MinDistance = FMath::Lerp(Foreground, FLinearColor::Green, 0.75f);
		const FLinearColor MaxDistance = FMath::Lerp(Foreground, FLinearColor::Red, 0.75f);

		return FMath::Lerp(MinDistance, MaxDistance, Distance);
	}

	static const FName ColumnName_ArcheType = TEXT("Archetype");
	static const FName ColumnName_Hash = TEXT("Hash");
	static const FName ColumnName_Entities = TEXT("Entities");

}

using FMassDebuggerArchetypeDataPtr = TSharedPtr<FMassDebuggerArchetypeData, ESPMode::ThreadSafe>;

//----------------------------------------------------------------------//
// SMassArchetypeTableRow
//----------------------------------------------------------------------//
class SMassArchetypeTableRow : public SMultiColumnTableRow<FMassDebuggerArchetypeDataPtr>
{
public:
	void Construct(const FArguments& InArgs, const TSharedPtr<STableViewBase>& InOwnerTableView, const FMassDebuggerArchetypeDataPtr InEntryItem, const FMassDebuggerArchetypeDataPtr InBaseItem, TSharedPtr<FMassDebuggerModel> InDebuggerModel)
	{
		Item = InEntryItem;
		DebuggerModel = InDebuggerModel;
		bIsBase = InEntryItem == InBaseItem;
		SMultiColumnTableRow<FMassDebuggerArchetypeDataPtr>::Construct(FSuperRowType::FArguments(), InOwnerTableView.ToSharedRef());
	}

	// SMultiColumnTableRow overrides
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		TSharedPtr<SHorizontalBox> Contents = SNew(SHorizontalBox);

		if (ColumnName == UE::Mass::Debugger::UI::Private::ColumnName_ArcheType)
		{
			Contents->AddSlot()
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(SExpanderArrow, SharedThis(this))
				.ShouldDrawWires(true)
				.IndentAmount(32)
				.BaseIndentLevel(0)
			];

			FLinearColor Color = FLinearColor::White;
			if (DebuggerModel->SelectedArchetypes.Num())
			{
				if (Item->bIsSelected)
				{
					const FLinearColor Foreground = FStyleColors::Foreground.GetSpecifiedColor();
					Color = FMath::Lerp(Foreground, FLinearColor::Green, 0.75f);
				}
				else
				{
					const float DistanceToSelected = DebuggerModel->MinDistanceToSelectedArchetypes(Item);
					Color = UE::Mass::Debugger::UI::Private::GetArchetypeDistanceColor(DistanceToSelected);
					if (Item->ArchetypeStats.EntitiesCount == 0)
					{
						Color.A *= 0.5f;
					}
				}
			}
			else
			{
				Color = FLinearColor::White;
				if (Item->ArchetypeStats.EntitiesCount == 0)
				{
					Color.A *= 0.5f;
				}
			}
			Contents->AddSlot()
			[
				SNew(STextBlock)
				.Text(Item->Label)
				.ToolTipText(Item->LabelTooltip)
				.ColorAndOpacity(Color)
			];

			if (bIsBase)
			{
				Contents->AddSlot()
				.VAlign(VAlign_Center)
				.Padding(FMargin(6,0))
				.AutoWidth()
				[
					SNew(SBorder)
					.BorderImage(FMassDebuggerStyle::GetBrush("MassDebug.Label.Background"))
					.Padding(FMargin(4, 1))
					[
						SNew(STextBlock)
						.TextStyle(FMassDebuggerStyle::Get(), TEXT("MassDebug.Label.Text"))
						.Text(LOCTEXT("ArchetypeDiffBase", "BASE"))
						.ToolTipText(LOCTEXT("ArchetypeDiffBaseTootip", "Base archetype used for diffing."))
					]
			];
			}
		}
		else if (ColumnName == UE::Mass::Debugger::UI::Private::ColumnName_Hash)
		{
			Contents->AddSlot()
				.Padding(5, 0)
				[
					SNew(STextBlock)
					.Font(FCoreStyle::GetDefaultFontStyle("Mono", 9))
					.Text(Item->HashLabel)
				];
		}
		else if (ColumnName == UE::Mass::Debugger::UI::Private::ColumnName_Entities)
		{
			Contents->AddSlot()
				.Padding(5, 0)
				[
					SNew(STextBlock)
					.Text(FText::FromString(FString::Printf(TEXT("%d"), Item->ArchetypeStats.EntitiesCount)))
					.Justification(ETextJustify::Right)
				];
		}

		return Contents.ToSharedRef();
	}

	FMassDebuggerArchetypeDataPtr Item;
	bool bIsBase = false;
	TSharedPtr<FMassDebuggerModel> DebuggerModel;
};

//----------------------------------------------------------------------//
// SMassArchetypesView
//----------------------------------------------------------------------//
void SMassArchetypesView::Construct(const FArguments& InArgs, TSharedRef<FMassDebuggerModel> InDebuggerModel)
{
	Initialize(InDebuggerModel);

	ChildSlot
		[
			SNew(SSplitter)
			.Orientation(Orient_Horizontal)
			
			+ SSplitter::Slot()
			.Value(.4f)
			.MinSize(350.0f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				[
					SAssignNew(ArchetypesTreeView, STreeView<TSharedPtr<FMassDebuggerArchetypeData>>)
					.TreeItemsSource(&TreeViewSource)
					.HeaderRow(SNew(SHeaderRow)
						+SHeaderRow::Column(UE::Mass::Debugger::UI::Private::ColumnName_ArcheType)
							.DefaultLabel(LOCTEXT("ArchetypeColumnHeader", "Archetype"))
							.FillWidth(0.70f)
						+SHeaderRow::Column(UE::Mass::Debugger::UI::Private::ColumnName_Hash)
							.DefaultLabel(LOCTEXT("HashColumnHeader", "Hash"))
							.FillSized(100)
						+SHeaderRow::Column(UE::Mass::Debugger::UI::Private::ColumnName_Entities)
							.DefaultLabel(LOCTEXT("EntitiesColumnHeader", "Entities"))
							.FillSized(60)
					)
					.OnGenerateRow_Lambda([this](TSharedPtr<FMassDebuggerArchetypeData> Item, const TSharedPtr<STableViewBase>& OwnerTable)
						{
							return SNew(SMassArchetypeTableRow, OwnerTable, Item, DiffBase, DebuggerModel);
						})
					.OnGetChildren_Lambda([](TSharedPtr<FMassDebuggerArchetypeData> InItem, TArray<TSharedPtr<FMassDebuggerArchetypeData>>& OutChildren)
						{
							if (InItem->Children.Num())
							{
								OutChildren.Append(InItem->Children);
							}
						})
					.OnSelectionChanged(this, &SMassArchetypesView::HandleSelectionChanged)
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(3)
				[
					SNew(STextBlock)
					.Text_Lambda([this]()
					{
						int32 ActiveCount = 0;
						int32 TotalCount = 0;
						for (const TSharedPtr<FMassDebuggerArchetypeData>& ArchetypeData : DebuggerModel->CachedArchetypeRepresentatives)
						{
							TotalCount++;
							if (ArchetypeData->ArchetypeStats.EntitiesCount > 0)
							{
								ActiveCount++;
							}
							
							for (const TSharedPtr<FMassDebuggerArchetypeData>& ChildArchetypeData : ArchetypeData->Children)
							{
								TotalCount++;
								if (ChildArchetypeData->ArchetypeStats.EntitiesCount > 0)
								{
									ActiveCount++;
								}
							}
						}
						
						return FText::Format(LOCTEXT("ArchetypeInfoFormat", "{0} archetypes, {1} active, {2} selected"),
							TotalCount, ActiveCount, DebuggerModel->SelectedArchetypes.Num());
					})
				]					
			]

			+ SSplitter::Slot()
			.Value(.6f)
			[
				SAssignNew(SelectedArchetypesView, SScrollBox)
			]
		];

	RebuildSelectedView();
}

void SMassArchetypesView::RebuildSelectedView()
{
	SelectedArchetypesView->ClearChildren();

	if (DebuggerModel->SelectedArchetypes.Num() > 1)
	{
		// print out entity count and memory totals
		int32 EntitiesCount = 0;
		int32 ChunksCount = 0;
		SIZE_T AllocatedSize = 0;
		SIZE_T WastedEntityMemory = 0;

		for (const TSharedPtr<FMassDebuggerArchetypeData>& ArchetypeData : DebuggerModel->SelectedArchetypes)
		{
			EntitiesCount += ArchetypeData->ArchetypeStats.EntitiesCount;
			ChunksCount += ArchetypeData->ArchetypeStats.ChunksCount;
			AllocatedSize += ArchetypeData->ArchetypeStats.AllocatedSize;
			WastedEntityMemory += ArchetypeData->ArchetypeStats.WastedEntityMemory;
		}

		float ChunkOccupancy = 0.f;
		for (const TSharedPtr<FMassDebuggerArchetypeData>& ArchetypeData : DebuggerModel->SelectedArchetypes)
		{
			if (ArchetypeData->ArchetypeStats.ChunksCount)
			{
				const float AverageEntitiesPerChunkActual = float(ArchetypeData->ArchetypeStats.EntitiesCount) / ArchetypeData->ArchetypeStats.ChunksCount;
				const float Occupancy = AverageEntitiesPerChunkActual / ArchetypeData->ArchetypeStats.EntitiesCountPerChunk;
				// @todo we're doing a simple proportional accumulation here. Percentiles would work better to avoid
				// outliers skewing the results. 
				ChunkOccupancy += Occupancy * ArchetypeData->ArchetypeStats.EntitiesCount / EntitiesCount;
			}
		}

		const FText TotalDescription = FText::Format(LOCTEXT("ArchetypesSelectionTotalsContent", "EntitiesCount: {0}\
				\nAllocated memory: {1}\
				\nWasted memory: {2} ({3}%)\
				\nTotal chunks: {4}\
				\nChunk occupancy: {5}")
			, FText::AsNumber(EntitiesCount), FText::AsMemory(AllocatedSize)
			, FText::AsMemory(WastedEntityMemory), FText::AsNumber(float(WastedEntityMemory) * 100.f / AllocatedSize)
			, FText::AsNumber(ChunksCount), FText::AsNumber(ChunkOccupancy));

		TSharedRef<SVerticalBox> Box = SNew(SVerticalBox);
		
		Box->AddSlot()
			.AutoHeight()
			.Padding(0, 4)
			[
				SNew(SRichTextBlock)
				.Text(LOCTEXT("ArchetypesSelectionTotals", "Total"))
				.DecoratorStyleSet(&FAppStyle::Get())
				.TextStyle(FAppStyle::Get(), "LargeText")
			];

		Box->AddSlot()
			.AutoHeight()
			.Padding(0, 4)
			[
				SNew(STextBlock)
				.Text(TotalDescription)
			];

		SelectedArchetypesView->AddSlot()
			.Padding(4)
			[
				Box
			];
	}

	const EMassBitSetDiffPrune Prune = DebuggerModel->SelectedArchetypes.Num() > 1 ? EMassBitSetDiffPrune::Same : EMassBitSetDiffPrune::None;
	
	for (const TSharedPtr<FMassDebuggerArchetypeData>& ArchetypeData : DebuggerModel->SelectedArchetypes)
	{
		SelectedArchetypesView->AddSlot()
			[
				SNew(SMassArchetype, ArchetypeData, DiffBase, Prune, DebuggerModel.ToSharedRef())
			];
	}
}

void SMassArchetypesView::HandleSelectionChanged(TSharedPtr<FMassDebuggerArchetypeData> InNode, ESelectInfo::Type InSelectInfo)
{
	if (!DebuggerModel || InSelectInfo == ESelectInfo::Direct)
	{
		return;
	}

	TArray<TSharedPtr<FMassDebuggerArchetypeData>> SelectedArchetypes;
	ArchetypesTreeView->GetSelectedItems(SelectedArchetypes);
	DebuggerModel->SelectArchetypes(SelectedArchetypes, InSelectInfo);
}

void SMassArchetypesView::OnRefresh()
{
	TreeViewSource.Reset();
	
	if (DebuggerModel)
	{
		TreeViewSource = DebuggerModel->CachedArchetypeRepresentatives;

		for (const TSharedPtr<FMassDebuggerArchetypeData>& ArchetypeData : TreeViewSource)
		{
			ArchetypesTreeView->SetItemExpansion(ArchetypeData, true);
		}
	}

	ArchetypesTreeView->RequestListRefresh();
	RebuildSelectedView();
}

void SMassArchetypesView::OnProcessorsSelected(TConstArrayView<TSharedPtr<FMassDebuggerProcessorData>> SelectedProcessors, ESelectInfo::Type SelectInfo)
{
	if (DebuggerModel)
	{
		OnArchetypesSelected(DebuggerModel->SelectedArchetypes, ESelectInfo::Direct);
	}
}

void SMassArchetypesView::OnArchetypesSelected(TConstArrayView<TSharedPtr<FMassDebuggerArchetypeData>> SelectedArchetypes, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo == ESelectInfo::Direct)
	{
		ArchetypesTreeView->ClearSelection();
		if (SelectedArchetypes.Num())
		{
			for (const TSharedPtr<FMassDebuggerArchetypeData>& ArchetypeData : SelectedArchetypes)
			{
				ArchetypesTreeView->SetItemSelection(ArchetypeData, true);
			}

			// scroll to the first item to make sure there's anything in view
			ArchetypesTreeView->RequestScrollIntoView(SelectedArchetypes[0]);
		}
	}

	DiffBase = nullptr;
	if (SelectedArchetypes.Num() == 1)
	{
		DiffBase = SelectedArchetypes[0]->Parent.Pin();
	}
	else if (SelectedArchetypes.Num() > 1)
	{
		DiffBase = SelectedArchetypes[0];
	}
	
	ArchetypesTreeView->RebuildList();
	RebuildSelectedView();
}

#undef LOCTEXT_NAMESPACE
