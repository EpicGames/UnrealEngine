// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "SMassDebuggerViewBase.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Layout/SScrollBox.h"

struct FMassDebuggerModel;
struct FMassDebuggerArchetypeData;
struct FMassDebuggerProcessorData;


class SMassArchetypesView : public SMassDebuggerViewBase
{
public:
	SLATE_BEGIN_ARGS(SMassArchetypesView)
	{
	}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<FMassDebuggerModel> InDebuggerModel);

	void HandleSelectionChanged(TSharedPtr<FMassDebuggerArchetypeData> InNode, ESelectInfo::Type InSelectInfo);

protected:
	virtual void OnRefresh() override;
	virtual void OnProcessorsSelected(TConstArrayView<TSharedPtr<FMassDebuggerProcessorData>> SelectedProcessors, ESelectInfo::Type SelectInfo) override;
	virtual void OnArchetypesSelected(TConstArrayView<TSharedPtr<FMassDebuggerArchetypeData>> SelectedArchetypes, ESelectInfo::Type SelectInfo) override;

	void RebuildSelectedView();
	
	TSharedPtr<STreeView<TSharedPtr<FMassDebuggerArchetypeData>>> ArchetypesTreeView;
	TSharedPtr<SScrollBox> SelectedArchetypesView;
	TSharedPtr<SHeaderRow> ArchetypesHeaderRow;
	TSharedPtr<FMassDebuggerArchetypeData> DiffBase;
	TArray<TSharedPtr<FMassDebuggerArchetypeData>> TreeViewSource;
};
