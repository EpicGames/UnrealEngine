// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "MassDebuggerStyle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"
#include "MassProcessingTypes.h"
#include "MassDebugger.h"
#include "MassDebuggerModel.h"

#define LOCTEXT_NAMESPACE "SMassDebugger"

struct FMassDebuggerQueryData;

enum class EMassBitSetDisplayMode 
{
	ReadOnly,
	ReadWrite,
	MAX
};

template<typename TBitSet>
class SMassBitSet : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMassBitSet)
		: _SlotPadding(5.f)
		, _TextColor(FLinearColor(1.f, 1.f, 1.f))
		, _BackgroundBrush(FMassDebuggerStyle::GetBrush("MassDebug.Fragment"))
		{}
		SLATE_ATTRIBUTE(FMargin, SlotPadding)
		SLATE_ATTRIBUTE(FLinearColor, TextColor)
		SLATE_ATTRIBUTE(const FSlateBrush*, BackgroundBrush)
	SLATE_END_ARGS()

	void Construct(const SMassBitSet::FArguments& InArgs, TSharedRef<FMassDebuggerModel> DebuggerModel, const FString& Label, const TBitSet& BitSet)
	{
		Construct(InArgs, DebuggerModel, Label, MakeArrayView(&BitSet, 1));
	}

	void Construct(const SMassBitSet::FArguments& InArgs, TSharedRef<FMassDebuggerModel> DebuggerModel, const FString& Label, TConstArrayView<TBitSet> BitSets, TConstArrayView<const FSlateBrush*> InBrushes = TConstArrayView<const FSlateBrush*>())
	{		
		/*Content = TEXT("<LargeText>Large test</>, <RichTextBlock.Bold>Bold</>");
		Content += BitSet.DebugGetStringDesc();*/
		TSharedRef<SWrapBox> ButtonBox = SNew(SWrapBox).UseAllottedSize(true);
		
		for (int i = 0; i < BitSets.Num(); ++i)
		{
			const FSlateBrush* Brush = InBrushes.IsValidIndex(i) ? InBrushes[i] : InArgs._BackgroundBrush.Get();
			AddBitSet(InArgs, ButtonBox, BitSets[i], Brush, DebuggerModel);
		}

		ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(150.0f)
				.Padding(FMargin(0.0f, InArgs._SlotPadding.Get().Top, 0.0f, InArgs._SlotPadding.Get().Bottom))
				[
					SNew(STextBlock)
					.Text(FText::FromString(Label))
				]
			]
			+ SHorizontalBox::Slot()
			[
				ButtonBox
			]
		];
	}

protected:
	void AddBitSet(const SMassBitSet::FArguments& InArgs, TSharedRef<SWrapBox>& ButtonBox, const TBitSet& BitSet, const FSlateBrush* Brush
		, TSharedRef<FMassDebuggerModel> DebuggerModel)
	{
#if WITH_MASSENTITY_DEBUG
		TArray<FName> TypeNames;
		BitSet.DebugGetIndividualNames(TypeNames);

		for (const FName& Name : TypeNames)
		{
			ButtonBox->AddSlot()
			.Padding(InArgs._SlotPadding)
			[
				SNew(SBorder)
				.ColorAndOpacity(InArgs._TextColor)
				.BorderImage(Brush)
				[
					SNew(SButton)
					.OnClicked_Lambda([Name, DebuggerModel]()
					{
						DebuggerModel->SelectFragment(Name);
						return FReply::Handled();
					})
					[
						SNew(STextBlock)
						.Text(FText::FromName(Name))
					]
				]
			];
		}
#endif // WITH_MASSENTITY_DEBUG
	}
};


enum class EMassBitSetDiffPrune : uint8
{
	None,	// No pruning
	Same,	// Prune tags that exists in base
};

template<typename TBitSet>
class SMassBitSetDiff : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMassBitSetDiff)
		: _SlotPadding(5.f)
		, _TextColor(FLinearColor(1.f, 1.f, 1.f))
		, _BackgroundBrush(FMassDebuggerStyle::GetBrush("MassDebug.Fragment"))
	{}
		SLATE_ATTRIBUTE(FMargin, SlotPadding)
		SLATE_ATTRIBUTE(FLinearColor, TextColor)
		SLATE_ATTRIBUTE(const FSlateBrush*, BackgroundBrush)
	SLATE_END_ARGS()

	void Construct(const SMassBitSetDiff::FArguments& InArgs, const FString& Label, const TBitSet& BaseBitSet, const TBitSet& BitSet
		, const EMassBitSetDiffPrune Prune, TSharedRef<FMassDebuggerModel> DebuggerModel)
	{
		TSharedRef<SWrapBox> ButtonBox = SNew(SWrapBox).UseAllottedSize(true);
		
		const FSlateBrush* Brush = InArgs._BackgroundBrush.Get();
		AddBitSet(InArgs, ButtonBox, BaseBitSet, BitSet, Brush, Prune, DebuggerModel);

		ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(150.0f)
				.Padding(FMargin(0.0f, InArgs._SlotPadding.Get().Top, 0.0f, InArgs._SlotPadding.Get().Bottom))
				[
					SNew(SButton)
					.OnClicked_Lambda([DebuggerModel, Label]()
					{
						DebuggerModel->SelectFragment(FName(*Label));
						return FReply::Handled();
					})
					[
						SNew(STextBlock)
						.Text(FText::FromString(Label))
					]
				]
			]
			+ SHorizontalBox::Slot()
			[
				ButtonBox
			]
		];
	}

protected:
	void AddBitSet(const SMassBitSetDiff::FArguments& InArgs, TSharedRef<SWrapBox>& ButtonBox, const TBitSet& BaseBitSet, const TBitSet& BitSet
		, const FSlateBrush* Brush, const EMassBitSetDiffPrune Prune, TSharedRef<FMassDebuggerModel> DebuggerModel)
	{
#if WITH_MASSENTITY_DEBUG
		TBitSet CombinedBitSet = BaseBitSet + BitSet;
		int32 SameCount = 0;

		for (auto It = CombinedBitSet.GetIndexIterator(); It; ++It)
		{
			const bool bInBase = BaseBitSet.IsBitSet(*It);
			const bool bInCurr = BitSet.IsBitSet(*It);

			if (Prune == EMassBitSetDiffPrune::Same && bInBase && bInCurr)
			{
				SameCount++;
				continue;
			}
			
			FString Name = CombinedBitSet.DebugGetStructTypeName(*It).ToString();
			FLinearColor BorderColor = FLinearColor::White;
			FLinearColor TextColor = InArgs._TextColor.Get();

			FText Tooltip;

			const FSlateBrush* Icon = nullptr;
			
			if (bInBase && bInCurr)
			{
				// Same
				BorderColor.A *= 0.6f;
				TextColor.A *= 0.75f;
				Tooltip = LOCTEXT("DiffExistsInBase", "Exists in base.");
			}
			else if (!bInBase && bInCurr)
			{
				// Added
				TextColor = FMath::Lerp(TextColor, FLinearColor::White, 0.9f);
				Icon = FAppStyle::Get().GetBrush("Icons.Plus");
				Tooltip = LOCTEXT("DiffAddedToBase", "Added to base.");
			}
			else
			{
				// Removed
				BorderColor.A *= 0.25f;
				TextColor.A *= 0.5f;
				Icon = FAppStyle::Get().GetBrush("Icons.Minus");
				Tooltip = LOCTEXT("DiffRemovedFromBase", "Removed from base.");
			}
			
			ButtonBox->AddSlot()
			.Padding(InArgs._SlotPadding)
			[
				SNew(SBorder)
				.BorderImage(Brush)
				.BorderBackgroundColor(BorderColor)
				.ColorAndOpacity(TextColor)
				.ToolTipText(Tooltip)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(3,0)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(Icon)
						.Visibility_Lambda([Icon] { return Icon != nullptr ? EVisibility::Visible : EVisibility::Collapsed; })
						.DesiredSizeOverride(FVector2D(10.0f, 10.0f))
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SButton)
						.OnClicked_Lambda([Name, DebuggerModel]()
							{
								DebuggerModel->SelectFragment(FName(*Name));
								return FReply::Handled();
							})
						[
							SNew(STextBlock)
								.Text(FText::FromString(Name))
						]
					]
				]
			];
		}

		if (SameCount > 0)
		{
			ButtonBox->AddSlot()
			.Padding(InArgs._SlotPadding)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(FText::Format(LOCTEXT("SameFragmentsFormat", "{0} same fragments"), FText::AsNumber(SameCount)))
				]
			];
		}

#endif // WITH_MASSENTITY_DEBUG
	}
};

namespace UE::Mass::Debugger::UI
{
	template<typename TBitSet>
	void AddBitSet(TSharedRef<SVerticalBox>& Box, const TBitSet& BitSetAccess, const FString& Label, const FSlateBrush* Brush, TSharedRef<FMassDebuggerModel> DebuggerModel)
	{
		if (BitSetAccess.IsEmpty() == false)
		{
			Box->AddSlot()
			.AutoHeight()
			.Padding(0, 8)
			[
				SNew(SMassBitSet<TBitSet>, DebuggerModel, Label, BitSetAccess)
				.BackgroundBrush(Brush)
				.SlotPadding(5.0f)
			];
		}
	}

	template<typename TBitSet>
	void AddBitSetDiff(TSharedRef<SVerticalBox>& Box, const TBitSet& BaseBitSet, const TBitSet& BitSet, const FString& Label, const FSlateBrush* Brush
		, const EMassBitSetDiffPrune Prune, TSharedRef<FMassDebuggerModel> DebuggerModel)
	{
		if (BaseBitSet.IsEmpty() == false || BitSet.IsEmpty() == false)
		{
			Box->AddSlot()
			.AutoHeight()
			.Padding(0, 8)
			[
				SNew(SMassBitSetDiff<TBitSet>, Label, BaseBitSet, BitSet, Prune, DebuggerModel)
				.BackgroundBrush(Brush)
				.SlotPadding(5.0f)
			];
		}
	}

} // UE::Mass::Debugger::UI

#undef LOCTEXT_NAMESPACE