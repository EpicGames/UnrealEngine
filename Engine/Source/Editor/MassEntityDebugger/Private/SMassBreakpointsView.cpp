// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMassBreakpointsView.h"
#include "MassDebugger.h"
#include "MassDebuggerModel.h"
#include "MassProcessor.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonWriter.h"
#include "Dom/JsonObject.h"

#define LOCTEXT_NAMESPACE "SMassDebugger"

void SMassBreakpointsView::Construct(const FArguments& InArgs, TSharedRef<FMassDebuggerModel> InModel)
{
	Initialize(InModel);

#if WITH_MASSENTITY_DEBUG
	InModel->OnEditorBreakpointsChangedDelegate.AddSP(this, &SMassBreakpointsView::RefreshBreakpoints);
	InModel->OnQueriesChangedDelegate.AddSP(this, &SMassBreakpointsView::RefreshBreakpoints);

	TriggerTypeOptions = {
		MakeShared<FString>(UE::Mass::Debug::FBreakpoint::TriggerTypeToString(
			UE::Mass::Debug::FBreakpoint::ETriggerType::None)),
		MakeShared<FString>(UE::Mass::Debug::FBreakpoint::TriggerTypeToString(
			UE::Mass::Debug::FBreakpoint::ETriggerType::ProcessorExecute)),
		MakeShared<FString>(UE::Mass::Debug::FBreakpoint::TriggerTypeToString(
			UE::Mass::Debug::FBreakpoint::ETriggerType::FragmentWrite)),
		MakeShared<FString>(UE::Mass::Debug::FBreakpoint::TriggerTypeToString(
			UE::Mass::Debug::FBreakpoint::ETriggerType::EntityCreate)),
		MakeShared<FString>(UE::Mass::Debug::FBreakpoint::TriggerTypeToString(
			UE::Mass::Debug::FBreakpoint::ETriggerType::EntityDestroy)),
		MakeShared<FString>(UE::Mass::Debug::FBreakpoint::TriggerTypeToString(
			UE::Mass::Debug::FBreakpoint::ETriggerType::FragmentAdd)),
		MakeShared<FString>(UE::Mass::Debug::FBreakpoint::TriggerTypeToString(
			UE::Mass::Debug::FBreakpoint::ETriggerType::FragmentRemove)),
		MakeShared<FString>(UE::Mass::Debug::FBreakpoint::TriggerTypeToString(
			UE::Mass::Debug::FBreakpoint::ETriggerType::TagAdd)),
		MakeShared<FString>(UE::Mass::Debug::FBreakpoint::TriggerTypeToString(
			UE::Mass::Debug::FBreakpoint::ETriggerType::TagRemove))
	};

	check(TriggerTypeOptions.Num() == static_cast<int32>(UE::Mass::Debug::FBreakpoint::ETriggerType::MAX));

	for (TSharedPtr<FString>& TypeStringPtr : TriggerTypeOptions)
	{
		UE::Mass::Debug::FBreakpoint::ETriggerType Trigger;
		if (TypeStringPtr && UE::Mass::Debug::FBreakpoint::StringToTriggerType(*TypeStringPtr, Trigger))
		{
			TriggerMap.Add(Trigger, TypeStringPtr);
		}
	}

	FilterTypeOptions = {
		MakeShared<FString>(UE::Mass::Debug::FBreakpoint::FilterTypeToString(
			UE::Mass::Debug::FBreakpoint::EFilterType::None)),
		MakeShared<FString>(UE::Mass::Debug::FBreakpoint::FilterTypeToString(
			UE::Mass::Debug::FBreakpoint::EFilterType::SpecificEntity)),
		MakeShared<FString>(UE::Mass::Debug::FBreakpoint::FilterTypeToString(
			UE::Mass::Debug::FBreakpoint::EFilterType::SelectedEntity)),
		MakeShared<FString>(UE::Mass::Debug::FBreakpoint::FilterTypeToString(
			UE::Mass::Debug::FBreakpoint::EFilterType::Query))
	};

	for (TSharedPtr<FString>& TypeStringPtr : FilterTypeOptions)
	{
		UE::Mass::Debug::FBreakpoint::EFilterType Filter;
		if (TypeStringPtr && UE::Mass::Debug::FBreakpoint::StringToFilterType(*TypeStringPtr, Filter))
		{
			FilterMap.Add(Filter, TypeStringPtr);
		}
	}
	
	BreakpointsListView = SNew(SListView<TSharedPtr<FMassDebuggerBreakpointData>>)
	.ListItemsSource(&InModel->CachedBreakpoints)
	.OnGenerateRow(this, &SMassBreakpointsView::OnGenerateBreakpointRow)
	.HeaderRow(
		SNew(SHeaderRow)
		+ SHeaderRow::Column("Enabled")
		.DefaultLabel(FText::GetEmpty())
		.FixedWidth(128)
		+ SHeaderRow::Column("TriggerType")
		.DefaultLabel(LOCTEXT("TriggerTypeColumn", "Trigger Type"))
		.FillWidth(0.2f)
		+ SHeaderRow::Column("Trigger")
		.DefaultLabel(LOCTEXT("TriggerColumn", "Trigger"))
		.FillWidth(0.2f)
		+ SHeaderRow::Column("FilterType")
		.DefaultLabel(LOCTEXT("FilterTypeColumn", "Filter Type"))
		.FillWidth(0.2f)
		+ SHeaderRow::Column("Filter")
		.DefaultLabel(LOCTEXT("FilterColumn", "Filter"))
		.FillWidth(0.2f)
		+ SHeaderRow::Column(TEXT("HitCount"))
		.DefaultLabel(LOCTEXT("HitCount", "Hit Count"))
		.FillWidth(0.2f)
	);

	ChildSlot
	[
		SNew(SVerticalBox)
    
		+ SVerticalBox::Slot().AutoHeight().Padding(4)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot().AutoWidth()
			[
				SNew(SButton)
				.ToolTipText(LOCTEXT("NewBreakpointTooltip", "New Breakpoint"))
				.OnClicked(this, &SMassBreakpointsView::NewBreakpointClicked)
				[
					SNew(SImage).Image(FAppStyle::GetBrush("Icons.PlusCircle"))
				]
			]
			+ SHorizontalBox::Slot().AutoWidth()
			[
				SNew(SButton)
				.ToolTipText(LOCTEXT("PasteBreakpointTooltip", "Paste Breakpoint"))
				.OnClicked(this, &SMassBreakpointsView::PasteBreakpointClicked)
				[
					SNew(SImage).Image(FAppStyle::GetBrush("GenericCommands.Paste"))
				]
			]
			+ SHorizontalBox::Slot().AutoWidth().Padding(0,0,4,0)
			[
				SNew(SButton)
				.ToolTipText(LOCTEXT("SaveBreakpointsTooltip", "Save Breakpoints"))
				.OnClicked(this, &SMassBreakpointsView::SaveBreakpointsClicked)
				[
					SNew(SImage).Image(FAppStyle::GetBrush("Icons.SaveModified"))
				]
			]
			+ SHorizontalBox::Slot().AutoWidth().Padding(0,0,4,0)
			[
				SNew(SButton)
				.ToolTipText(LOCTEXT("ClearAllBreakpoints", "Clear All Breakpoints"))
				.OnClicked(this, &SMassBreakpointsView::ClearBreakpointsClicked)
				[
					SNew(SImage).Image(FAppStyle::GetBrush("Icons.Delete"))
				]
			]
		]
		+ SVerticalBox::Slot().FillHeight(1.f).Padding(4)
		[
			BreakpointsListView.ToSharedRef()
		]
	];
#else
	ChildSlot
		[
			SNew(STextBlock).Text(LOCTEXT("DebugNotEnabled", "Mass Entity Debugging Not Enabled"))
		];
#endif
}

void SMassBreakpointsView::RefreshBreakpoints()
{
#if WITH_MASSENTITY_DEBUG
	if (BreakpointsListView.IsValid())
	{
		BreakpointsListView->RebuildList();
	}
#endif
}

FReply SMassBreakpointsView::ClearBreakpointsClicked()
{
#if WITH_MASSENTITY_DEBUG
	if (DebuggerModel)
	{
		DebuggerModel->RemoveAllBreakpoints();
	}
#endif
	return FReply::Handled();
}

FReply SMassBreakpointsView::SaveBreakpointsClicked()
{
#if WITH_MASSENTITY_DEBUG
	if (DebuggerModel)
	{
		DebuggerModel->SaveBreakpointsToDisk();
	}
#endif
	return FReply::Handled();
}

FReply SMassBreakpointsView::PasteBreakpointClicked()
{
#if WITH_MASSENTITY_DEBUG
	if (DebuggerModel)
	{
		FString ClipboardText;
		FPlatformApplicationMisc::ClipboardPaste(ClipboardText);

		DebuggerModel->CreateBreakpointFromString(ClipboardText);
	}
#endif
	return FReply::Handled();
}

FReply SMassBreakpointsView::NewBreakpointClicked()
{
#if WITH_MASSENTITY_DEBUG
	if (DebuggerModel)
	{
		DebuggerModel->CreateBreakpoint();
	}
#endif
	return FReply::Handled();
}

TSharedRef<ITableRow> SMassBreakpointsView::OnGenerateBreakpointRow(
    TSharedPtr<FMassDebuggerBreakpointData> InItem,
    const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SMassBreakpointsView::SBreakpointTableRow, OwnerTable, SharedThis(this), InItem.ToSharedRef(), DebuggerModel.ToSharedRef());
}

void SMassBreakpointsView::SBreakpointTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TSharedRef<SMassBreakpointsView> InParentView, TSharedRef<FMassDebuggerBreakpointData> InBreakpointData, TSharedRef<FMassDebuggerModel> InModel)
{
	BreakpointData = InBreakpointData;
	DebuggerModel = InModel;
	ParentView = InParentView;
	SMultiColumnTableRow<TSharedPtr<FMassDebuggerBreakpointData>>::Construct(SMultiColumnTableRow<TSharedPtr<FMassDebuggerBreakpointData>>::FArguments(), InOwnerTableView);
}

TSharedRef<SWidget> SMassBreakpointsView::SBreakpointTableRow::GenerateWidgetForColumn(const FName& ColumnId)
{
#if WITH_MASSENTITY_DEBUG
	TSharedPtr<FMassDebuggerBreakpointData> Data = BreakpointData.Pin();
	TSharedPtr<SMassBreakpointsView> View = ParentView.Pin();

	if (!Data || !View)
	{
		return SNullWidget::NullWidget;
	}

	if (ColumnId == "Enabled")
	{
		return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2)
		[
			SNew(SCheckBox)
				.IsChecked_Lambda([WeakItem = BreakpointData]()
				{
					if (TSharedPtr<FMassDebuggerBreakpointData> Item = WeakItem.Pin())
					{
						return Item->BreakpointInstance.bEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					}
					return ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([WeakItem = BreakpointData, WeakModel = DebuggerModel](ECheckBoxState NewState)
				{
					if (TSharedPtr<FMassDebuggerBreakpointData> Item = WeakItem.Pin())
					{
						Item->BreakpointInstance.bEnabled = (NewState == ECheckBoxState::Checked);
						if (TSharedPtr<FMassDebuggerModel> Model = WeakModel.Pin())
						{
							Model->ApplyBreakpointsToCurrentEnvironment();
						}
					}
				})
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2)
		[
			SNew(SButton)
				.ToolTipText(LOCTEXT("DeleteBreakpointTooltip", "Delete Breakpoint"))
				.OnClicked_Lambda([WeakItem = BreakpointData, WeakModel = DebuggerModel]()
				{
					if (TSharedPtr<FMassDebuggerBreakpointData> Item = WeakItem.Pin())
					{
						if (TSharedPtr<FMassDebuggerModel> Model = WeakModel.Pin())
						{
							Model->RemoveBreakpoint(Item->BreakpointInstance.Handle);
						}
					}
					return FReply::Handled();
				})
			[
				SNew(SImage).Image(FAppStyle::GetBrush("Icons.Delete"))
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2)
		[
			SNew(SButton)
			.ToolTipText(LOCTEXT("CopyBreakpointTooltip", "Copy Breakpoint"))
			.OnClicked_Lambda([WeakItem = BreakpointData]()
			{
				if (TSharedPtr<FMassDebuggerBreakpointData> Item = WeakItem.Pin())
				{
					FString JsonString;
					const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&JsonString);
					FJsonSerializer::Serialize(Item->SerializeToJson().ToSharedRef(), Writer);

					FPlatformApplicationMisc::ClipboardCopy(*JsonString);
				}
				return FReply::Handled();
			})
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("GenericCommands.Copy"))
			]
		];
	}
    else if (ColumnId == "TriggerType")
    {
        return SNew(SComboBox<TSharedPtr<FString>>)
        .OptionsSource(&View->TriggerTypeOptions)
        .OnGenerateWidget_Lambda([](TSharedPtr<FString> Str)
		{
			return SNew(STextBlock).Text(FText::FromString(*Str));
		})
        .OnSelectionChanged_Lambda([WeakData = BreakpointData, WeakModel = DebuggerModel](TSharedPtr<FString> Str, ESelectInfo::Type)
        {
            UE::Mass::Debug::FBreakpoint::ETriggerType NewType;
			TSharedPtr<FMassDebuggerBreakpointData> Data = WeakData.Pin();
            if (Data && Str && UE::Mass::Debug::FBreakpoint::StringToTriggerType(*Str, NewType))
            {
				Data->BreakpointInstance.TriggerType = NewType;
				if (TSharedPtr<FMassDebuggerModel> Model = WeakModel.Pin())
				{
					Model->ApplyBreakpointsToCurrentEnvironment();
				}
            }
        })
        .InitiallySelectedItem(View->TriggerMap[Data->BreakpointInstance.TriggerType])
        [
            SNew(STextBlock)
			.Text_Lambda([WeakData = BreakpointData]()
			{
				if (TSharedPtr<FMassDebuggerBreakpointData> Data = WeakData.Pin())
				{
					return FText::FromString(UE::Mass::Debug::FBreakpoint::TriggerTypeToString(Data->BreakpointInstance.TriggerType));
				}
				return LOCTEXT("BreakpointInvalid", "Invalid Breakpoint");
			})
        ];
    }
    else if (ColumnId == "Trigger")
    {
        if (Data->BreakpointInstance.TriggerType == UE::Mass::Debug::FBreakpoint::ETriggerType::ProcessorExecute)
        {
            return SNew(SSearchableComboBox)
            .OptionsSource(&DebuggerModel.Pin()->ProcessorNames)
            .OnGenerateWidget_Lambda([](TSharedPtr<FString> Str)
			{
				if (Str)
				{
					return SNew(STextBlock)
						.Text(FText::FromString(*Str));
				}
				return SNew(STextBlock)
					.Text(LOCTEXT("ProcessorNone", "None"));
			})
            .OnSelectionChanged_Lambda([WeakData = BreakpointData, WeakModel = DebuggerModel](TSharedPtr<FString> Str, ESelectInfo::Type)
            {
				if (TSharedPtr<FMassDebuggerBreakpointData> Data = WeakData.Pin())
				{
					if (Str)
					{
						Data->TriggerName = *Str;
					}
					else
					{
						Data->TriggerName.Empty();
					}

					if (TSharedPtr<FMassDebuggerModel> Model = WeakModel.Pin())
					{
						Data->ReconcileDataFromNames(*Model);
						Model->ApplyBreakpointsToCurrentEnvironment();
					}
				}
            })
            [
                SNew(STextBlock).Text_Lambda([WeakData = BreakpointData]()
                {
					if (TSharedPtr<FMassDebuggerBreakpointData> Data = WeakData.Pin())
					{
						return FText::FromString(Data->TriggerName);
					}
                    return LOCTEXT("SelectProcessor","Select Processor");
                })
            ];
        }
        else if (Data->BreakpointInstance.TriggerType == UE::Mass::Debug::FBreakpoint::ETriggerType::FragmentWrite
			|| Data->BreakpointInstance.TriggerType == UE::Mass::Debug::FBreakpoint::ETriggerType::FragmentAdd
			|| Data->BreakpointInstance.TriggerType == UE::Mass::Debug::FBreakpoint::ETriggerType::FragmentRemove)
        {
            return SNew(SSearchableComboBox)
            .OptionsSource(&DebuggerModel.Pin()->FragmentNames)
            .OnGenerateWidget_Lambda([](TSharedPtr<FString> Str)
			{
				if (Str)
				{
					return SNew(STextBlock)
						.Text(FText::FromString(*Str));
				}
				return SNew(STextBlock)
					.Text(LOCTEXT("FragmentNone", "None"));
			})
            .OnSelectionChanged_Lambda([WeakData = BreakpointData, WeakModel = DebuggerModel](TSharedPtr<FString> Str, ESelectInfo::Type)
            {
				if (TSharedPtr<FMassDebuggerBreakpointData> Data = WeakData.Pin())
				{
					if (Str)
					{
						Data->TriggerName = *Str;
					}
					else
					{
						Data->TriggerName.Empty();
					}

					if (TSharedPtr<FMassDebuggerModel> Model = WeakModel.Pin(); !Model->IsStale())
					{
						Data->ReconcileDataFromNames(*Model);
						Data->ApplyToEngine(*Model);
					}
				}
            })
            [
				SNew(STextBlock).Text_Lambda([WeakData = BreakpointData]()
				{
					if (TSharedPtr<FMassDebuggerBreakpointData> Data = WeakData.Pin())
					{
						return FText::FromString(Data->TriggerName);
					}
					return LOCTEXT("SelectFragment", "Select Fragment");
				})
            ];
        }
		else if (Data->BreakpointInstance.TriggerType == UE::Mass::Debug::FBreakpoint::ETriggerType::TagAdd
			|| Data->BreakpointInstance.TriggerType == UE::Mass::Debug::FBreakpoint::ETriggerType::TagRemove)
        {
            return SNew(SSearchableComboBox)
            .OptionsSource(&DebuggerModel.Pin()->TagNames)
            .OnGenerateWidget_Lambda([](TSharedPtr<FString> Str)
			{
				if (Str)
				{
					return SNew(STextBlock)
						.Text(FText::FromString(*Str));
				}
				return SNew(STextBlock)
					.Text(LOCTEXT("TagNone", "None"));
			})
            .OnSelectionChanged_Lambda([WeakData = BreakpointData, WeakModel = DebuggerModel](TSharedPtr<FString> Str, ESelectInfo::Type)
            {
				if (TSharedPtr<FMassDebuggerBreakpointData> Data = WeakData.Pin())
				{
					if (Str)
					{
						Data->TriggerName = *Str;
					}
					else
					{
						Data->TriggerName.Empty();
					}

					if (TSharedPtr<FMassDebuggerModel> Model = WeakModel.Pin(); !Model->IsStale())
					{
						Data->ReconcileDataFromNames(*Model);
						Data->ApplyToEngine(*Model);
					}
				}
            })
            [
				SNew(STextBlock).Text_Lambda([WeakData = BreakpointData]()
				{
					if (TSharedPtr<FMassDebuggerBreakpointData> Data = WeakData.Pin())
					{
						return FText::FromString(Data->TriggerName);
					}
					return LOCTEXT("SelectTag", "Select Tag");
				})
            ];
        }
        return SNullWidget::NullWidget;
    }
    else if (ColumnId == "FilterType")
	{
        return SNew(SComboBox<TSharedPtr<FString>>)
        .OptionsSource(&View->FilterTypeOptions)
        .OnGenerateWidget_Lambda([](TSharedPtr<FString> Str)
		{
			return SNew(STextBlock).Text(FText::FromString(*Str));
		})
        .OnSelectionChanged_Lambda([WeakData = BreakpointData, WeakModel = DebuggerModel](TSharedPtr<FString> Str, ESelectInfo::Type)
        {
            
			if (TSharedPtr<FMassDebuggerBreakpointData> Data = WeakData.Pin())
			{
				UE::Mass::Debug::FBreakpoint::EFilterType NewType;
				if (UE::Mass::Debug::FBreakpoint::StringToFilterType(*Str, NewType))
				{
					Data->BreakpointInstance.FilterType = NewType;
					if (TSharedPtr<FMassDebuggerModel> Model = WeakModel.Pin())
					{
						Model->ApplyBreakpointsToCurrentEnvironment();
					}
				}
			}
        })
        .InitiallySelectedItem(View->FilterMap[Data->BreakpointInstance.FilterType])
        [
            SNew(STextBlock)
			.Text_Lambda([WeakData = BreakpointData]()
			{
				if (TSharedPtr<FMassDebuggerBreakpointData> Data = WeakData.Pin())
				{
					return FText::FromString(UE::Mass::Debug::FBreakpoint::FilterTypeToString(Data->BreakpointInstance.FilterType));
				}
				return LOCTEXT("BreakpointInvalid", "Invalid Breakpoint");
			})
        ];
    }
    else if (ColumnId == "Filter")
    {
		if (Data->BreakpointInstance.FilterType == UE::Mass::Debug::FBreakpoint::EFilterType::Query)
		{
			return SNew(SSearchableComboBox)
            .OptionsSource(&DebuggerModel.Pin()->QueryNames)
            .OnGenerateWidget_Lambda([](TSharedPtr<FString> Str)
			{
				if (Str)
				{
					return SNew(STextBlock)
						.Text(FText::FromString(*Str));
				}
				return SNew(STextBlock)
					.Text(LOCTEXT("QueryNone", "None"));
			})
            .OnSelectionChanged_Lambda([WeakData = BreakpointData, WeakModel = DebuggerModel](TSharedPtr<FString> Str, ESelectInfo::Type)
            {
				if (TSharedPtr<FMassDebuggerBreakpointData> Data = WeakData.Pin())
				{
					if (Str)
					{
						Data->FilterName = *Str;
					}
					else
					{
						Data->FilterName.Empty();
					}

					if (TSharedPtr<FMassDebuggerModel> Model = WeakModel.Pin())
					{
						Data->ReconcileDataFromNames(*Model);
						Model->ApplyBreakpointsToCurrentEnvironment();
					}
				}
            })
            [
                SNew(STextBlock)
				.Text_Lambda([WeakData = BreakpointData]()
                {
					if (TSharedPtr<FMassDebuggerBreakpointData> Data = WeakData.Pin())
					{
						if (!Data->FilterName.IsEmpty())
						{
							return FText::FromString(Data->FilterName);
						}
					}
                    return LOCTEXT("SelectQuery","Select Query");
                })
            ];
		}
    }
	else if (ColumnId == "HitCount")
	{
		return SAssignNew(HitCount, STextBlock)
		.Text_Lambda([WeakData = BreakpointData, WeakModel = DebuggerModel]()
		{
			if (TSharedPtr<FMassDebuggerBreakpointData> Data = WeakData.Pin())
			{
				if (TSharedPtr<FMassDebuggerModel> Model = WeakModel.Pin(); !Model->IsStale())
				{
					UE::Mass::Debug::FBreakpoint* EngineBreak = FMassDebugger::FindBreakpoint(*Model->Environment->GetEntityManager(), Data->BreakpointInstance.Handle);
					if (EngineBreak)
					{
						return FText::AsNumber(EngineBreak->HitCount);
					}
				}
			}
			return FText::AsNumber(0);
		});
	}
#endif //WITH_MASSENTITY_DEBUG
    return SNullWidget::NullWidget;
}


void SMassBreakpointsView::SBreakpointTableRow::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SMultiColumnTableRow<TSharedPtr<FMassDebuggerBreakpointData>>::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (HitCount.IsValid())
	{
		HitCount->Invalidate(EInvalidateWidgetReason::Paint);
	}
}

#undef LOCTEXT_NAMESPACE
