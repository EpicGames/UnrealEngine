// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "SMassDebuggerViewBase.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "MassDebugger.h"
#include "MassDebuggerBreakpoints.h"

class SSearchableComboBox;
struct FMassDebuggerModel;
struct FMassDebuggerProcessorData;
struct FMassDebuggerFragmentData;
struct FMassDebuggerBreakpointData;

class SMassBreakpointsView : public SMassDebuggerViewBase
{
public:
	SLATE_BEGIN_ARGS(SMassBreakpointsView)
	{
	}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<FMassDebuggerModel> InModel);

protected:
	// SMassDebuggerViewBase overrides
	virtual void OnRefresh() override
	{
		RefreshBreakpoints();
	}

	virtual void OnProcessorsSelected(TConstArrayView<TSharedPtr<FMassDebuggerProcessorData>> SelectedProcessors, ESelectInfo::Type SelectInfo)
	{
	}

	virtual void OnArchetypesSelected(TConstArrayView<TSharedPtr<FMassDebuggerArchetypeData>> SelectedArchetypes, ESelectInfo::Type SelectInfo)
	{
	}

private:
	friend class SBreakpointTableRow;
	class SBreakpointTableRow : public SMultiColumnTableRow<TSharedPtr<FMassDebuggerBreakpointData>>
	{
	public:
		SLATE_BEGIN_ARGS(SBreakpointTableRow)
		{
		}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TSharedRef<SMassBreakpointsView> InParentView, TSharedRef<FMassDebuggerBreakpointData> InBreakpointData, TSharedRef<FMassDebuggerModel> InModel);
		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;
		virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	private:
		TSharedPtr<STextBlock> HitCount;

		TWeakPtr<FMassDebuggerBreakpointData> BreakpointData;
		TWeakPtr<FMassDebuggerModel> DebuggerModel;
		TWeakPtr<SMassBreakpointsView> ParentView;
	};

	void RefreshBreakpoints();
	FReply ClearBreakpointsClicked();
	FReply SaveBreakpointsClicked();
	FReply PasteBreakpointClicked();
	FReply NewBreakpointClicked();

	TSharedRef<ITableRow> OnGenerateBreakpointRow(
		TSharedPtr<FMassDebuggerBreakpointData> InItem,
		const TSharedRef<STableViewBase>& OwnerTable);

	TSharedPtr<SListView<TSharedPtr<FMassDebuggerBreakpointData>>> BreakpointsListView;

	TArray<TSharedPtr<FString>> TriggerTypeOptions;
	TMap<UE::Mass::Debug::FBreakpoint::ETriggerType, TSharedPtr<FString>> TriggerMap;
	TArray<TSharedPtr<FString>> FilterTypeOptions;
	TMap<UE::Mass::Debug::FBreakpoint::EFilterType, TSharedPtr<FString>> FilterMap;
};
