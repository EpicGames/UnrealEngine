// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMassDebugger.h"
#include "SMassBreakpointsView.h"
#include "SMassProcessorsView.h"
#include "SMassProcessingView.h"
#include "SMassArchetypesView.h"
#include "SMassEntitiesView.h"
#include "SMassFragmentsView.h"
#include "SMassQueryEditorView.h"
#include "MassDebuggerModel.h"
#include "Engine/Engine.h"
#include "CoreGlobals.h"
#include "Framework/Docking/LayoutService.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SComboBox.h"
#if WITH_EDITOR
#include "Editor.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "SMassDebugger"

//----------------------------------------------------------------------//
// FMassDebuggerCommands
//----------------------------------------------------------------------//
FMassDebuggerCommands::FMassDebuggerCommands()
	: TCommands<FMassDebuggerCommands>("MassDebugger", LOCTEXT("MassDebuggerName", "Mass Debugger"), NAME_None, "MassDebuggerStyle")
{ }

void FMassDebuggerCommands::RegisterCommands() 
{
	UI_COMMAND(RefreshData, "RecacheData", "Recache data", EUserInterfaceActionType::Button, FInputChord());
}

namespace UE::Mass::Debugger::Private
{
	const FLazyName ToolbarTabId("Toolbar");
	const FLazyName BreakpointsTabId("Breakpoints");
	const FLazyName ProcessorsTabId("Processors");
	const FLazyName ProcessingGraphTabId("Processing Graphs");
	const FLazyName FragmentsTabId("Fragments");
	const FLazyName ArchetypesTabId("Archetypes");
	const FLazyName EntitiesTabId("Entities");
	const FLazyName QueryEditorTabId("QueryEditor");

	bool IsSupportedWorldType(const EWorldType::Type WorldType)
	{
		return 
			WorldType == EWorldType::Game || 
			WorldType == EWorldType::Editor || 
			WorldType == EWorldType::PIE;
	}
}

static TAutoConsoleVariable<bool> CVarMassDebuggerAutoSwitchToNewEnvironments(
	TEXT("MassDebugger.AutoSwitchToNewEnvironments"),
	false,
	TEXT("If enabled, the Mass Debugger will automatically switch to newly registered environments."),
	ECVF_Default
);

//----------------------------------------------------------------------//
// SMassDebugger
//----------------------------------------------------------------------//
SMassDebugger::SMassDebugger()
	: SCompoundWidget(), CommandList(MakeShareable(new FUICommandList))
	, DebuggerModel(MakeShareable(new FMassDebuggerModel))
{
}

SMassDebugger::~SMassDebugger()
{
#if WITH_MASSENTITY_DEBUG
	FMassDebugger::OnEntityManagerInitialized.Remove(OnEntityManagerInitializedHandle);
	FMassDebugger::OnEntityManagerDeinitialized.Remove(OnEntityManagerDeinitializedHandle);
	FMassDebugger::OnProcessorProviderRegistered.Remove(OnProcessorProviderRegisteredHandle);
#endif // WITH_MASSENTITY_DEBUG
}

bool SMassDebugger::IsAutoSwitchEnvironmentsEnabled()
{
	return CVarMassDebuggerAutoSwitchToNewEnvironments.GetValueOnAnyThread();
}

void SMassDebugger::Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow)
{
	using namespace UE::Mass::Debugger::Private;

	BindDelegates();

	const FMassDebuggerCommands& Commands = FMassDebuggerCommands::Get();
	FUICommandList& ActionList = *CommandList;

	ActionList.MapAction(Commands.RefreshData
		, FExecuteAction::CreateSP(this, &SMassDebugger::RefreshData)
		, FCanExecuteAction::CreateSP(this, &SMassDebugger::CanRefreshData));

	// Tab Spawners
	TabManager = FGlobalTabmanager::Get()->NewTabManager(ConstructUnderMajorTab);
	TSharedRef<FWorkspaceItem> AppMenuGroup = TabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("MassDebuggerGroupName", "Mass Debugger"));

	TabManager->RegisterTabSpawner(ToolbarTabId, FOnSpawnTab::CreateRaw(this, &SMassDebugger::SpawnToolbar))
		.SetDisplayName(LOCTEXT("ToolbarTabTitle", "Toolbar"))
		.SetGroup(AppMenuGroup);

	TabManager->RegisterTabSpawner(ProcessorsTabId, FOnSpawnTab::CreateRaw(this, &SMassDebugger::SpawnProcessorsTab))
		.SetDisplayName(LOCTEXT("ProcessorsTabTitle", "Processors"))
		.SetGroup(AppMenuGroup);

	TabManager->RegisterTabSpawner(ProcessingGraphTabId, FOnSpawnTab::CreateRaw(this, &SMassDebugger::SpawnProcessingTab))
		.SetDisplayName(LOCTEXT("ProcessingTabTitle", "Processing Graphs"))
		.SetGroup(AppMenuGroup);

	TabManager->RegisterTabSpawner(FragmentsTabId, FOnSpawnTab::CreateRaw(this, &SMassDebugger::SpawnFragmentsTab))
		.SetDisplayName(LOCTEXT("FragmentsTabTitle", "Fragments"))
		.SetGroup(AppMenuGroup);

	TabManager->RegisterTabSpawner(ArchetypesTabId, FOnSpawnTab::CreateRaw(this, &SMassDebugger::SpawnArchetypesTab))
		.SetDisplayName(LOCTEXT("ArchetypesTabTitle", "Archetypes"))
		.SetGroup(AppMenuGroup);

	TabManager->RegisterTabSpawner(BreakpointsTabId, FOnSpawnTab::CreateRaw(this, &SMassDebugger::SpawnBreakpointsTab))
		.SetDisplayName(LOCTEXT("BreakpointsTabTitle", "Breakpoints"))
		.SetGroup(AppMenuGroup);

	TabManager->RegisterTabSpawner(EntitiesTabId, FOnSpawnTab::CreateRaw(this, &SMassDebugger::SpawnEntitiesTab))
		.SetDisplayName(LOCTEXT("EntitiesTabTitle", "Entities"))
		.SetGroup(AppMenuGroup);

	TabManager->RegisterTabSpawner(QueryEditorTabId, FOnSpawnTab::CreateRaw(this, &SMassDebugger::SpawnQueryEditorTab))
		.SetDisplayName(LOCTEXT("QueryEditorTabTitle", "Query Editor"))
		.SetGroup(AppMenuGroup);

	// Default Layout
	TSharedRef<FTabManager::FLayout> Layout = CreateDefaultLayout();

	Layout = FLayoutSaveRestore::LoadFromConfig(GEditorLayoutIni, Layout);

	ChildSlot
	[
		TabManager->RestoreFrom(Layout, ConstructUnderWindow).ToSharedRef()
	];

	TabManager->SetOnPersistLayout(
		FTabManager::FOnPersistLayout::CreateStatic(
			[](const TSharedRef<FTabManager::FLayout>& InLayout)
			{
				if (InLayout->GetPrimaryArea().Pin().IsValid())
				{
					FLayoutSaveRestore::SaveToConfig(GEditorLayoutIni, InLayout);
				}
			}
		)
	);

	DebuggerModel->DebuggerWindow = SharedThis(this).ToWeakPtr();
}

TSharedRef<FTabManager::FLayout> SMassDebugger::CreateDefaultLayout()
{
	using namespace UE::Mass::Debugger::Private;
	TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout("MassDebuggerLayout_v1.3")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewStack()
				->AddTab(ToolbarTabId, ETabState::OpenedTab)
				->SetHideTabWell(true)
			)
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Vertical)
					->Split
					(
						FTabManager::NewStack()
						->AddTab(ProcessorsTabId, ETabState::OpenedTab)
						->AddTab(ProcessingGraphTabId, ETabState::OpenedTab)
						->AddTab(FragmentsTabId, ETabState::OpenedTab)
						->AddTab(QueryEditorTabId, ETabState::OpenedTab)
						->SetForegroundTab(ProcessorsTabId.Resolve())
					)
					->Split
					(
						FTabManager::NewStack()
						->AddTab(BreakpointsTabId, ETabState::OpenedTab)
						->SetForegroundTab(BreakpointsTabId.Resolve())
					)
				)
				->Split
				(
					FTabManager::NewStack()
					->AddTab(ArchetypesTabId, ETabState::OpenedTab)
					->AddTab(EntitiesTabId, ETabState::OpenedTab)
					->SetForegroundTab(ArchetypesTabId.Resolve())
				)
			)
		);
	return Layout;
}

TSharedRef<SDockTab> SMassDebugger::SpawnToolbar(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> MajorTab = SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		.ShouldAutosize(true);

	FSlimHorizontalToolBarBuilder ToolBarBuilder(CommandList, FMultiBoxCustomization::None);
	ToolBarBuilder.BeginSection(TEXT("Window"));
	{
		ToolBarBuilder.AddComboButton(
			FUIAction()
			, FOnGetContent::CreateSP(this, &SMassDebugger::GenerateWindowMenu)
			, LOCTEXT("MassDebuggerWindowMenu_Label", "Window")
			, LOCTEXT("MassWindowMenu_Tooltip", "Window options")
			, FSlateIcon()
			, false
		);
	}
	ToolBarBuilder.EndSection();
	ToolBarBuilder.BeginSection(TEXT("Debugger"));
	{
		ToolBarBuilder.AddToolBarButton(FMassDebuggerCommands::Get().RefreshData, NAME_None, LOCTEXT("RefreshData", "Refresh"), LOCTEXT("RefreshDebuggerTooltip", "Refreshes data cached by the debugger instance"));//, FSlateIcon(FLogVisualizerStyle::Get().GetStyleSetName(), TEXT("Toolbar.Update")));
	}

	RebuildEnvironmentsList();
	
	MajorTab->SetContent(
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		[
			ToolBarBuilder.MakeWidget()
		]
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.Padding(2.f)
		.AutoWidth()
		[
			SNew(SCheckBox)
			.IsChecked_Lambda([this]()
			{
				return CVarMassDebuggerAutoSwitchToNewEnvironments.GetValueOnAnyThread()
					? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			})
			.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState)
			{
				CVarMassDebuggerAutoSwitchToNewEnvironments->Set(NewState == ECheckBoxState::Checked, ECVF_SetByConsole);
			})
			.ToolTipText(LOCTEXT("AutoSwitchEnvironments_Tooltip", "Automatically switch to new environments as they are registered"))
			[
				SNew(STextBlock).Text(LOCTEXT("AutoSwitchEnvironments_Label", "Auto-Switch"))
			]
		]
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.Padding(2.f)
		.AutoWidth()
		[
			SAssignNew(EnvironmentComboBox, SComboBox<TSharedPtr<FMassDebuggerEnvironment>>)
			.OptionsSource(&EnvironmentsList)
			.OnGenerateWidget_Lambda([](TSharedPtr<FMassDebuggerEnvironment> Item)
				{
					check(Item);
					return SNew(STextBlock).Text(FText::FromString(Item->GetDisplayName()));
				})
			.OnSelectionChanged(this, &SMassDebugger::HandleEnvironmentChanged)
			.ToolTipText(LOCTEXT("Environment_Tooltip", "Pick where to get the data from"))
			[
				SAssignNew(EnvironmentComboLabel, STextBlock)
				.Text(LOCTEXT("PickEnvironment", "Pick Environment"))
			]
		]
	);

	return MajorTab;
}

TSharedRef<SWidget> SMassDebugger::GenerateWindowMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	FUIAction ProcessorsUIAction(FExecuteAction::CreateSP(this, &SMassDebugger::ShowProcessorView));
	FUIAction ArchetypesUIAction(FExecuteAction::CreateSP(this, &SMassDebugger::ShowArchetypesView));
	FUIAction ProcessingGraphsUIAction(FExecuteAction::CreateSP(this, &SMassDebugger::ShowProcessingGraphsView));
	FUIAction FragmentsUIAction(FExecuteAction::CreateSP(this, &SMassDebugger::ShowFragmentsView));
	FUIAction BreakpointsUIAction(FExecuteAction::CreateSP(this, &SMassDebugger::ShowBreakpointsView));
	FUIAction EntitesUIAction(FExecuteAction::CreateSP(this, &SMassDebugger::ShowEntitesView));
	FUIAction QueryEditorUIAction(FExecuteAction::CreateSP(this, &SMassDebugger::ShowQueryEditorView));
	FUIAction ResetLayoutUIAction(FExecuteAction::CreateSP(this, &SMassDebugger::ResetLayout));

	MenuBuilder.BeginSection(TEXT("Tabs"));
	MenuBuilder.AddMenuEntry(LOCTEXT("ProcessorsTabLabel", "Processors"), LOCTEXT("ProcessorsTabTooltip", "Show Processors Tab"), FSlateIcon(), ProcessorsUIAction);
	MenuBuilder.AddMenuEntry(LOCTEXT("ArchetypesTabLabel", "Archetypes"), LOCTEXT("ArchtypesTabTooltip", "Show Archtypes Tab"), FSlateIcon(), ArchetypesUIAction);
	MenuBuilder.AddMenuEntry(LOCTEXT("ProcessingGraphsTabLabel", "Processing Graphs"), LOCTEXT("ProcessingGraphsTabTooltip", "Show Processing Graphs Tab"), FSlateIcon(), ProcessingGraphsUIAction);
	MenuBuilder.AddMenuEntry(LOCTEXT("FragmentsTabLabel", "Fragments"), LOCTEXT("FragmentsTabTooltip", "Show Fragments"), FSlateIcon(), FragmentsUIAction);
	MenuBuilder.AddMenuEntry(LOCTEXT("BreakpointsTabLabel", "Breakpoints"), LOCTEXT("BreakpointsTabTooltip", "Show Breakpoints Tab"), FSlateIcon(), BreakpointsUIAction);
	MenuBuilder.AddMenuEntry(LOCTEXT("EntitiesTabLabel", "Entities"), LOCTEXT("EntitiesTabTooltip", "Show Entities Tab"), FSlateIcon(), EntitesUIAction);
	MenuBuilder.AddMenuEntry(LOCTEXT("QueryEditorTabLabel", "Query Editor"), LOCTEXT("QueryEditoTabTooltip", "Show Query Editor Tab"), FSlateIcon(), QueryEditorUIAction);
	MenuBuilder.AddSeparator();
	MenuBuilder.AddMenuEntry(LOCTEXT("ResetLayoutLabel", "Reset Layout"), LOCTEXT("ResetLayoutTooltip", "Reset the Mass Debugger Layout"), FSlateIcon(), ResetLayoutUIAction);

	return MenuBuilder.MakeWidget();
}

void SMassDebugger::OnEntityManagerInitialized(const FMassEntityManager& EntityManager)
{
	const UWorld* World = EntityManager.GetWorld();
	if (World && UE::Mass::Debugger::Private::IsSupportedWorldType(World->WorldType))
	{
		const int32 Index = EnvironmentsList.Add(MakeShareable(new FMassDebuggerEnvironment(EntityManager.AsShared())));
		if (EnvironmentComboBox.IsValid())
		{
			EnvironmentComboBox->RefreshOptions();
			if (IsAutoSwitchEnvironmentsEnabled())
			{
				EnvironmentComboBox->SetSelectedItem(EnvironmentsList[Index]);
				HandleEnvironmentChanged(EnvironmentsList[Index], ESelectInfo::Type::Direct);
			}
		}
	}
}

#if WITH_MASSENTITY_DEBUG
void SMassDebugger::OnProcessorProviderRegistered(const FMassDebugger::FEnvironment& Environment)
{
	const int32 FoundIndex = EnvironmentsList.FindLastByPredicate([WeakManager = Environment.EntityManager](const TSharedPtr<FMassDebuggerEnvironment>& TestedEnvironment)
	{
		return TestedEnvironment->EntityManager == WeakManager;
	});
	
	if (ensureMsgf(FoundIndex != INDEX_NONE, TEXT("We never expect OnProcessorProviderRegistered called for an Environment that has not been registered")))
	{
		EnvironmentsList[FoundIndex]->ProcessorProviders = Environment.ProcessorProviders;
	}
}
#endif // WITH_MASSENTITY_DEBUG

void SMassDebugger::OnEntityManagerDeinitialized(const FMassEntityManager& EntityManager)
{
	const UWorld* World = EntityManager.GetWorld();
	if (World != nullptr && UE::Mass::Debugger::Private::IsSupportedWorldType(World->WorldType) == false)
	{
		return;
	}

	if (EntityManager.DoesSharedInstanceExist())
	{
		FMassDebuggerEnvironment InEnvironment(EntityManager.AsShared());

		if (EnvironmentsList.RemoveAll([&InEnvironment](const TSharedPtr<FMassDebuggerEnvironment>& Element)
			{
				return *Element.Get() == InEnvironment;
			}) > 0)
		{
			if (DebuggerModel->IsCurrentEnvironment(InEnvironment))
			{
				DebuggerModel->MarkAsStale();
				EnvironmentComboLabel->SetText(DebuggerModel->GetDisplayName());
			}
		}
	}
	else
	{
		// EntityManager is either undergoing destruction or it has never been made sharable
		// all we can do right now is remove all no longer valid environments
		if (EnvironmentsList.RemoveAll([](const TSharedPtr<FMassDebuggerEnvironment>& Element)
			{
				check(Element);
				return Element.Get()->EntityManager.IsValid() == false;
			}) > 0)
		{
			if (DebuggerModel->IsCurrentEnvironmentValid() == false)
			{
				DebuggerModel->MarkAsStale();
				EnvironmentComboLabel->SetText(DebuggerModel->GetDisplayName());
			}
		}
	}
	if (EnvironmentComboBox.IsValid())
	{
		EnvironmentComboBox->RefreshOptions();
	}
}

void SMassDebugger::HandleEnvironmentChanged(TSharedPtr<FMassDebuggerEnvironment> Item, ESelectInfo::Type SelectInfo)
{
	DebuggerModel->SetEnvironment(Item);
	EnvironmentComboLabel->SetText(DebuggerModel->GetDisplayName());
}

void SMassDebugger::RebuildEnvironmentsList()
{
	EnvironmentsList.Reset();
#if WITH_MASSENTITY_DEBUG
	for (const FMassDebugger::FEnvironment& Environment : FMassDebugger::GetEnvironments())
	{
		if (const FMassEntityManager* EntityManagerPtr = Environment.EntityManager.Pin().Get())
		{
			const UWorld* World = EntityManagerPtr->GetWorld();
			if (World == nullptr || UE::Mass::Debugger::Private::IsSupportedWorldType(World->WorldType))
			{
				EnvironmentsList.Add(MakeShareable(new FMassDebuggerEnvironment(EntityManagerPtr->AsShared())));
				EnvironmentsList.Last()->ProcessorProviders = Environment.ProcessorProviders;
			}
		}
	}
#endif // WITH_MASSENTITY_DEBUG
}

TSharedRef<SDockTab> SMassDebugger::SpawnProcessorsTab(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> MajorTab = SNew(SDockTab)
		.TabRole(ETabRole::PanelTab);

	TSharedPtr<SWidget> TabContent = SNew(SMassProcessorsView, DebuggerModel);
	MajorTab->SetContent(TabContent.ToSharedRef());

	return MajorTab;
}

TSharedRef<SDockTab> SMassDebugger::SpawnProcessingTab(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> MajorTab = SNew(SDockTab)
		.TabRole(ETabRole::PanelTab);

	TSharedPtr<SWidget> TabContent = SNew(SMassProcessingView, DebuggerModel);
	MajorTab->SetContent(TabContent.ToSharedRef());

	return MajorTab;
}

TSharedRef<SDockTab> SMassDebugger::SpawnFragmentsTab(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> MajorTab = SNew(SDockTab)
		.TabRole(ETabRole::PanelTab);

	TSharedPtr<SWidget> TabContent = SNew(UE::MassDebugger::FragmentsView::SFragmentsView, DebuggerModel);
	MajorTab->SetContent(TabContent.ToSharedRef());

	return MajorTab;
}

TSharedRef<SDockTab> SMassDebugger::SpawnArchetypesTab(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> MajorTab = SNew(SDockTab)
		.TabRole(ETabRole::PanelTab);

	TSharedPtr<SWidget> TabContent = SNew(SMassArchetypesView, DebuggerModel);
	MajorTab->SetContent(TabContent.ToSharedRef());

	return MajorTab;
}

TSharedRef<SDockTab> SMassDebugger::SpawnBreakpointsTab(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> MajorTab = SNew(SDockTab)
		.TabRole(ETabRole::PanelTab);

	TSharedPtr<SWidget> TabContent = SNew(SMassBreakpointsView, DebuggerModel);
	MajorTab->SetContent(TabContent.ToSharedRef());

	return MajorTab;
}

TSharedRef<SDockTab> SMassDebugger::SpawnEntitiesTab(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> MajorTab = SNew(SDockTab)
		.TabRole(ETabRole::PanelTab);

	TSharedPtr<SWidget> TabContent = SNew(SMassEntitiesView, DebuggerModel, 0);
	MajorTab->SetContent(TabContent.ToSharedRef());

	return MajorTab;
}

TSharedRef<SDockTab> SMassDebugger::SpawnQueryEditorTab(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> MajorTab = SNew(SDockTab)
		.TabRole(ETabRole::PanelTab);

	TSharedPtr<SWidget> TabContent = SNew(UE::MassDebugger::SQueryEditorView, DebuggerModel);
	MajorTab->SetContent(TabContent.ToSharedRef());

	return MajorTab;
}

bool SMassDebugger::CanRefreshData()
{
	return DebuggerModel->HasEnvironmentSelected();
}

void SMassDebugger::RefreshData()
{
	if (DebuggerModel->IsStale() && EnvironmentComboBox.IsValid() && EnvironmentComboLabel.IsValid())
	{
		EnvironmentComboBox->RefreshOptions();
		EnvironmentComboLabel->SetText(LOCTEXT("PickEnvironment", "Pick Environment"));
	}
	DebuggerModel->RefreshAll();
}

void SMassDebugger::ShowProcessorView() const
{
	TabManager->TryInvokeTab(UE::Mass::Debugger::Private::ProcessorsTabId.Resolve());
}

void SMassDebugger::ShowArchetypesView() const
{
	TabManager->TryInvokeTab(UE::Mass::Debugger::Private::ArchetypesTabId.Resolve());
}

void SMassDebugger::ShowBreakpointsView() const
{
	TabManager->TryInvokeTab(UE::Mass::Debugger::Private::BreakpointsTabId.Resolve());
}

void SMassDebugger::ShowProcessingGraphsView() const
{
	TabManager->TryInvokeTab(UE::Mass::Debugger::Private::ProcessingGraphTabId.Resolve());
}

void SMassDebugger::ShowFragmentsView() const
{
	TabManager->TryInvokeTab(UE::Mass::Debugger::Private::FragmentsTabId.Resolve());
}

void SMassDebugger::ShowEntitesView() const
{
	TabManager->TryInvokeTab(UE::Mass::Debugger::Private::EntitiesTabId.Resolve());
}

void SMassDebugger::ShowQueryEditorView() const
{
	TabManager->TryInvokeTab(UE::Mass::Debugger::Private::QueryEditorTabId.Resolve());
}

void SMassDebugger::ResetLayout()
{
	TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(AsShared());
	if (Window.IsValid())
	{
		ChildSlot.DetachWidget();
		ChildSlot.AttachWidget(TabManager->RestoreFrom(CreateDefaultLayout(), Window).ToSharedRef());
	}
}

void SMassDebugger::ShowSelectedView() const
{
	
}

void SMassDebugger::BindDelegates()
{
#if WITH_MASSENTITY_DEBUG
	OnEntityManagerInitializedHandle = FMassDebugger::OnEntityManagerInitialized.AddRaw(this, &SMassDebugger::OnEntityManagerInitialized);
	OnEntityManagerDeinitializedHandle = FMassDebugger::OnEntityManagerDeinitialized.AddRaw(this, &SMassDebugger::OnEntityManagerDeinitialized);
	OnProcessorProviderRegisteredHandle = FMassDebugger::OnProcessorProviderRegistered.AddRaw(this, &SMassDebugger::OnProcessorProviderRegistered);
#endif // WITH_MASSENTITY_DEBUG
}

#undef LOCTEXT_NAMESPACE
