// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/Docking/SDockTab.h"
#include "Framework/Commands/Commands.h"
#include "MassDebugger.h"

struct FMassDebuggerEnvironment;
struct FMassDebuggerModel;
struct FMassEntityManager;
struct FMassProcessingPhaseManager;
template<typename T> class SComboBox;


class FMassDebuggerCommands : public TCommands<FMassDebuggerCommands>
{
public:
	/** Default constructor. */
	FMassDebuggerCommands();

public:
	// TCommands interface
	virtual void RegisterCommands() override;
public:

	TSharedPtr<FUICommandInfo> RefreshData;
};

class SMassDebuggerTab : public SDockTab
{
public:
	virtual bool SupportsKeyboardFocus() const override { return true; }
};

class SMassDebugger : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMassDebugger) 
	{
	}
	SLATE_END_ARGS()

public:
	SMassDebugger();
	virtual ~SMassDebugger();

	/**
	* Constructs the application.
	*
	* @param InArgs The Slate argument list.
	* @param ConstructUnderMajorTab The major tab which will contain the debugger
	* @param ConstructUnderWindow The window in which this widget is being constructed.
	*/
	void Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow);

	virtual bool SupportsKeyboardFocus() const override { return true; }

	void ShowProcessorView() const;
	void ShowArchetypesView() const;
	void ShowBreakpointsView() const;
	void ShowProcessingGraphsView() const;
	void ShowFragmentsView() const;
	void ShowEntitesView() const;
	void ResetLayout();
	void ShowSelectedView() const;
	void ShowQueryEditorView() const;

	static bool IsAutoSwitchEnvironmentsEnabled();

protected:
	TSharedRef<SDockTab> SpawnToolbar(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnProcessorsTab(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnProcessingTab(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnFragmentsTab(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnArchetypesTab(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnBreakpointsTab(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnEntitiesTab(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnQueryEditorTab(const FSpawnTabArgs& Args);

	TSharedRef<FTabManager::FLayout> CreateDefaultLayout();
	TSharedRef<SWidget> GenerateWindowMenu();

	void BindDelegates();
	void OnEntityManagerInitialized(const FMassEntityManager& EntityManager);
	void OnEntityManagerDeinitialized(const FMassEntityManager& EntityManager);
	void HandleEnvironmentChanged(TSharedPtr<FMassDebuggerEnvironment> Item, ESelectInfo::Type SelectInfo);
	void RebuildEnvironmentsList();

	bool CanRefreshData();
	void RefreshData();

#if WITH_MASSENTITY_DEBUG
void OnProcessorProviderRegistered(const FMassDebugger::FEnvironment& Environment);
#endif // WITH_MASSENTITY_DEBUG

	// Holds the list of UI commands.
	TSharedRef<FUICommandList> CommandList;

	// Holds the tab manager that manages the front-end's tabs.
	TSharedPtr<FTabManager> TabManager;

	TSharedPtr<SComboBox<TSharedPtr<FMassDebuggerEnvironment>>> EnvironmentComboBox;
	TSharedPtr<STextBlock> EnvironmentComboLabel;

	TArray<TSharedPtr<FMassDebuggerEnvironment>> EnvironmentsList;

	TSharedRef<FMassDebuggerModel> DebuggerModel;

	FDelegateHandle OnEntityManagerInitializedHandle;
	FDelegateHandle OnEntityManagerDeinitializedHandle;
	FDelegateHandle OnProcessorProviderRegisteredHandle;
};
