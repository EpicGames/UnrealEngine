// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMassDebuggerViewBase.h"
#include "MassDebuggerModel.h"

//----------------------------------------------------------------------//
// SMassDebuggerViewBase
//----------------------------------------------------------------------//
SMassDebuggerViewBase::~SMassDebuggerViewBase()
{
	if (DebuggerModel)
	{
		DebuggerModel->OnRefreshDelegate.Remove(OnRefreshHandle);
		DebuggerModel->OnProcessorsSelectedDelegate.Remove(OnProcessorsSelectedHandle);
		DebuggerModel->OnArchetypesSelectedDelegate.Remove(OnArchetypesSelectedHandle);
	}
}

void SMassDebuggerViewBase::Initialize(TSharedRef<FMassDebuggerModel> InDebuggerModel)
{
	DebuggerModel = InDebuggerModel;

	OnRefreshHandle = DebuggerModel->OnRefreshDelegate.AddRaw(this, &SMassDebuggerViewBase::OnRefresh);
	OnProcessorsSelectedHandle = DebuggerModel->OnProcessorsSelectedDelegate.AddRaw(this, &SMassDebuggerViewBase::OnProcessorsSelected);
	OnArchetypesSelectedHandle = DebuggerModel->OnArchetypesSelectedDelegate.AddRaw(this, &SMassDebuggerViewBase::OnArchetypesSelected);
}
