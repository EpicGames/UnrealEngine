// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Types/SlateEnums.h"

struct FMassDebuggerProcessorData;
struct FMassDebuggerArchetypeData;
struct FMassDebuggerModel;

class SMassDebuggerViewBase : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMassDebuggerViewBase){}
	SLATE_END_ARGS()

	virtual ~SMassDebuggerViewBase();

protected:
	void Initialize(TSharedRef<FMassDebuggerModel> InDebuggerModel);

	virtual void OnRefresh() = 0;
	virtual void OnProcessorsSelected(TConstArrayView<TSharedPtr<FMassDebuggerProcessorData>> SelectedProcessors, ESelectInfo::Type SelectInfo) = 0;
	virtual void OnArchetypesSelected(TConstArrayView<TSharedPtr<FMassDebuggerArchetypeData>> SelectedArchetypes, ESelectInfo::Type SelectInfo) = 0;

	FDelegateHandle OnRefreshHandle;
	FDelegateHandle OnProcessorsSelectedHandle;
	FDelegateHandle OnArchetypesSelectedHandle;

	TSharedPtr<FMassDebuggerModel> DebuggerModel;
};
