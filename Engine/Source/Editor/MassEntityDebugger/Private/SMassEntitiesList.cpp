// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMassEntitiesList.h"
#include "MassDebuggerModel.h"
#include "MassEntityTypes.h"
#include "MassEntityView.h"
#include "SMassBitSet.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/SListView.h"
#include "MassArchetypeData.h"
#include "PropertyEditorModule.h"
#include "Modules/ModuleManager.h"
#include "UObject/StructOnScope.h"
#include "IStructureDetailsView.h"
#include "ISinglePropertyView.h"
#include "IPropertyRowGenerator.h"
#include "IDetailTreeNode.h"
#include "DetailTreeNode.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "SMassDebugger"

//----------------------------------------------------------------------//
// SMassEntitiesList
//----------------------------------------------------------------------//

namespace UE::MassDebugger::EntitiesList::Private
{
	const FLazyName ColumnHandle = TEXT("EntityHandle");
}

void SMassEntitiesList::Construct(const SMassEntitiesList::FArguments& InArgs, TSharedRef<FMassDebuggerModel> InDebuggerModel)
{
#if WITH_MASSENTITY_DEBUG
	FragmentSelectBox = SNew(SBox);

	DebuggerModel = InDebuggerModel;

	ChildSlot
	[
		SNew(SBorder)
		.Padding(5.0f)
		[
			SAssignNew(TreeView, STreeView<EntitiesTableRowPtr>)
			.SelectionMode(ESelectionMode::None)
			.TreeItemsSource(&GridRows)
			.OnGetChildren(this, &SMassEntitiesList::TreeView_OnGetChildren)
			.OnGenerateRow(this, &SMassEntitiesList::TreeView_OnGenerateRow)
			.HeaderRow
			(
				SAssignNew(TreeViewHeaderRow, SHeaderRow)
				.Visibility(EVisibility::Visible)
			)
		]
	];

	UpdateTreeColumns();
	SetEntities(InArgs._Entities);

#else
	ChildSlot
	[
		SNew(STextBlock)
			.Text(LOCTEXT("MassEntityDebuggingNotEnabled", "Mass Entity Debugging Not Enabled for this configuration"))
	];
#endif
}


void SMassEntitiesList::BuildGrid()
{
#if WITH_MASSENTITY_DEBUG
	PopulateGridColumns();
	TreeView->RequestTreeRefresh();
#endif
}

void SMassEntitiesList::AddPropertyRecursive(TSharedPtr<SHorizontalBox> HBox, TSharedPtr<SVerticalBox> VBox, TSharedPtr<IPropertyHandle> Prop, bool bShowName)
{
#if WITH_MASSENTITY_DEBUG
		uint32 NumChildren = 0;
		Prop->GetNumChildren(NumChildren);
		if (NumChildren > 0)
		{
			VBox = SNew(SVerticalBox);
			HBox->AddSlot()
			.AutoWidth()
			.HAlign(EHorizontalAlignment::HAlign_Center)
			.VAlign(EVerticalAlignment::VAlign_Top)
			[
				VBox.ToSharedRef()
			];
			if (bShowName)
			{
				VBox->AddSlot()
				.AutoHeight()
				.HAlign(EHorizontalAlignment::HAlign_Center)
				.VAlign(EVerticalAlignment::VAlign_Top)
				[
					Prop->CreatePropertyNameWidget()
				];
			}

			for (uint32 i = 0; i < NumChildren; i++)
			{
				AddPropertyRecursive(HBox, VBox, Prop->GetChildHandle(i), true);
			}
		}
		else
		{
			if (bShowName)
			{
				VBox->AddSlot()
				.AutoHeight()
				.HAlign(EHorizontalAlignment::HAlign_Left)
				.VAlign(EVerticalAlignment::VAlign_Top)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(EHorizontalAlignment::HAlign_Left)
					.VAlign(EVerticalAlignment::VAlign_Top)
					[
						Prop->CreatePropertyNameWidget()
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(EHorizontalAlignment::HAlign_Left)
					.VAlign(EVerticalAlignment::VAlign_Top)
					[
						Prop->CreatePropertyValueWidget()
					]
				];
			}
			else
			{
				VBox->AddSlot()
				.HAlign(EHorizontalAlignment::HAlign_Left)
				.VAlign(EVerticalAlignment::VAlign_Top)
				[
					Prop->CreatePropertyValueWidget()
				];
			}
		}
#endif
}

void SMassEntitiesList::SetEntities(const TArray<FMassEntityHandle>& InEntities)
{
#if WITH_MASSENTITY_DEBUG

	AvailableFragmentNames.Reset();
	// ensure that the selected fragments are preserved even if they're not present on the new set of entites:
	AvailableFragmentNames.Append(SelectedFragmentNames);
	
	GridRows.Reset();
	GridRows.SetNum(InEntities.Num(), EAllowShrinking::No);

	if (DebuggerModel.IsValid() && DebuggerModel->Environment.IsValid() && DebuggerModel->Environment->EntityManager.IsValid())
	{
		const FMassEntityManager& EntityManager = *(DebuggerModel->Environment->EntityManager.Pin());
		TSet<FMassArchetypeHandle> SearchedArchetypes;
		const TWeakPtr<SMassEntitiesList> WeakThisPtr = StaticCastWeakPtr<SMassEntitiesList>(AsWeak());

		for (int i = 0; i < InEntities.Num(); i++)
		{
			if (!GridRows[i].IsValid())
			{
				GridRows[i] = MakeShared<FGridRow>();
			}
			const FMassEntityHandle& Entity = InEntities[i];
			GridRows[i]->Entity = Entity;
			GridRows[i]->EntitiesList = WeakThisPtr;

			FMassArchetypeHandle ArchetypeHandle = EntityManager.GetArchetypeForEntity(Entity);
			
			if (ArchetypeHandle.IsValid() && !SearchedArchetypes.Contains(ArchetypeHandle))
			{
				SearchedArchetypes.Add(ArchetypeHandle);
				EntityManager.ForEachArchetypeFragmentType(ArchetypeHandle, [this](const UScriptStruct* FragmentType)
				{
					AvailableFragmentNames.AddUnique(FragmentType->GetFName());
				});

				const FMassArchetypeSharedFragmentValues& SharedFragments = FMassDebugger::GetSharedFragmentValues(EntityManager, Entity);
				const TArray<FConstSharedStruct>& ConstSharedStructs = SharedFragments.GetConstSharedFragments();
				const TArray<FSharedStruct>& SharedStructs = SharedFragments.GetSharedFragments();

				for (const FSharedStruct& SharedStruct : SharedStructs)
				{
					AvailableFragmentNames.AddUnique(SharedStruct.GetScriptStruct()->GetFName());
				}

				for (const FConstSharedStruct& ConstSharedStruct : ConstSharedStructs)
				{
					AvailableFragmentNames.AddUnique(ConstSharedStruct.GetScriptStruct()->GetFName());
				}
			}
		}
	}

	AvailableFragmentNames.Sort([](const FName& A, const FName& B) { return A.Compare(B) < 0; });

	CreateFragmentSelectDropdown();
	BuildGrid();
#endif
}

TSharedPtr<IPropertyHandle> FindPropertyHandle(const FProperty* Property, TArray<TSharedRef<IDetailTreeNode>>& NodesToSearch)
{
#if WITH_MASSENTITY_DEBUG
	TArray<TSharedRef<IDetailTreeNode>> Children;
	while (NodesToSearch.Num() > 0)
	{
		TSharedRef<IDetailTreeNode> CurNode = NodesToSearch.Pop(EAllowShrinking::No);
		if (CurNode->GetNodeType() == EDetailNodeType::Item)
		{
			TSharedPtr<IPropertyHandle> PropertyHandle = CurNode->CreatePropertyHandle();
			if (PropertyHandle.IsValid() && PropertyHandle->GetProperty() == Property)
			{
				return PropertyHandle;
			}
		}

		CurNode->GetChildren(Children, true);
		NodesToSearch.Append(Children);
	}
#endif
	return nullptr;
}

void SMassEntitiesList::RefreshFragmentData()
{
#if WITH_MASSENTITY_DEBUG
	if (DebuggerModel.IsValid() && !DebuggerModel->IsStale())
	{
		const FMassEntityManager& EntityManager = *(DebuggerModel->Environment->EntityManager.Pin());

		for (EntitiesTableRowPtr& RowPtr : GridRows)
		{
			if (RowPtr.IsValid())
			{
				FGridRow& Row = *RowPtr;

				for (FGridRow::FFragmentInfo& Info : Row.FragmentInfo)
				{
					if (UE::Mass::IsA<FMassFragment>(Info.StructType))
					{
						if (Info.StructData)
						{
							FMassDebugger::GetFragmentData(EntityManager, Info.StructType, Row.Entity, Info.StructData);
						}
						else
						{
							Info.StructData = FMassDebugger::GetFragmentData(EntityManager, Info.StructType, Row.Entity);
						}
					}
					else if (UE::Mass::IsA<FMassSharedFragment>(Info.StructType))
					{
						if (Info.StructData)
						{
							FMassDebugger::GetSharedFragmentData(EntityManager, Info.StructType, Row.Entity, Info.StructData);
						}
						else
						{
							Info.StructData = FMassDebugger::GetSharedFragmentData(EntityManager, Info.StructType, Row.Entity);
						}
						
					}
					else if (UE::Mass::IsA<FMassConstSharedFragment>(Info.StructType))
					{
						if (Info.StructData)
						{
							FMassDebugger::GetConstSharedFragmentData(EntityManager, Info.StructType, Row.Entity, Info.StructData);
						}
						else
						{
							Info.StructData = FMassDebugger::GetConstSharedFragmentData(EntityManager, Info.StructType, Row.Entity);
						}
					}
					else
					{
						ensureMsgf(false, TEXT("Invalid entity data type(%s)"), *Info.StructType->GetDisplayNameText().ToString());
					}
				}
			}
		}
	}
#endif
}

void SMassEntitiesList::PopulateGridColumns()
{
#if WITH_MASSENTITY_DEBUG
	Columns.Reset();
	ColumnIndexByID.Reset();

	for (FName FragmentName : SelectedFragmentNames)
	{
		const UScriptStruct* FragmentStructType = FMassDebugger::GetFragmentTypeFromName(FragmentName);
		if (!FragmentStructType)
		{
			continue;
		}

		FMassEntitiesListColumn& Column = Columns.AddDefaulted_GetRef();
		Column.ColumnID = FName(FragmentStructType->GetName());
		Column.StructType = FragmentStructType;
		Column.ColumnLabel = FragmentStructType->GetName();
		ColumnIndexByID.Add(Column.ColumnID, Columns.Num() - 1);

		for (TFieldIterator<FProperty> It(FragmentStructType); It; ++It)
		{
			FProperty* Property = *It;

			FMassEntitiesListColumn& PropertyColumn = Columns.AddDefaulted_GetRef();
			PropertyColumn.StructType = FragmentStructType;
			PropertyColumn.Property = Property;
			PropertyColumn.ColumnID = FName(FString::Printf(TEXT("%s_%s"), *FragmentStructType->GetName(), *Property->GetName()));
			PropertyColumn.ColumnLabel = Property->GetName();
			ColumnIndexByID.Add(PropertyColumn.ColumnID, Columns.Num() - 1);
		}
	}

	UpdateTreeColumns();

#endif
}

FReply SMassEntitiesList::OnClearAllSelectedFragmentsClicked()
{
	SelectedFragmentTypes.Reset();
	SelectedFragmentNames.Reset();
	BuildGrid();
	return FReply::Handled();
}

void SMassEntitiesList::OnFragmentCheckStateChanged(ECheckBoxState NewState, FName FragmentName)
{
#if WITH_MASSENTITY_DEBUG
	const bool bIsChecked = (NewState == ECheckBoxState::Checked);
	if (bIsChecked)
	{
		SelectedFragmentNames.AddUnique(FragmentName);
		SelectedFragmentNames.Sort([](const FName& A, const FName& B) { return A.Compare(B) < 0; });
	}
	else
	{
		SelectedFragmentNames.Remove(FragmentName);
	}

	SelectedFragmentTypes.Reset(SelectedFragmentNames.Num());
	for (FName SelectedFragmentName : SelectedFragmentNames)
	{
		SelectedFragmentTypes.Add(FMassDebugger::GetFragmentTypeFromName(SelectedFragmentName));
	}
	
	for (EntitiesTableRowPtr& Row : GridRows)
	{
		Row->bDirty = true;
	}

	BuildGrid();
#endif
}

ECheckBoxState SMassEntitiesList::GetFragmentCheckState(FName FragmentName) const
{
	if (SelectedFragmentNames.Contains(FragmentName))
	{
		return ECheckBoxState::Checked;
	}
	return ECheckBoxState::Unchecked;
}

void SMassEntitiesList::CreateFragmentSelectDropdown()
{
#if WITH_MASSENTITY_DEBUG
	TSharedPtr<SVerticalBox> DropdownContent = SNew(SVerticalBox);

	DropdownContent->AddSlot()
	.AutoHeight()
	.Padding(5.0f)
	[
		SNew(SButton)
			.Text(LOCTEXT("ClearAll", "Clear All"))
			.OnClicked(FOnClicked::CreateSP(this, &SMassEntitiesList::OnClearAllSelectedFragmentsClicked))
	];
	
	TSharedPtr<SScrollBox> FragmentScrollBox = SNew(SScrollBox);
	DropdownContent->AddSlot()
	.FillHeight(1.)
	.Padding(5.0f)
	[
		FragmentScrollBox.ToSharedRef()
	];

	TSharedPtr<SVerticalBox> FragmentList = SNew(SVerticalBox);
	FragmentScrollBox->AddSlot()
	[
		FragmentList.ToSharedRef()
	];

	for (const FName& FragmentName : AvailableFragmentNames)
	{
		FragmentList->AddSlot()
		.AutoHeight()
		.Padding(5.0f)
		[
			SNew(SCheckBox)
			.OnCheckStateChanged(FOnCheckStateChanged::CreateSP(this, &SMassEntitiesList::OnFragmentCheckStateChanged, FragmentName))
			.IsChecked(TAttribute<ECheckBoxState>::CreateSP(this, &SMassEntitiesList::GetFragmentCheckState, FragmentName))
			.Content()
			[
				SNew(STextBlock).Text(FText::FromName(FragmentName))
			]
		];
	}

	TSharedRef<SComboButton> FragmentsButton = SNew(SComboButton)
	.ButtonContent()
	[
		SNew(STextBlock)
		.Text(FText::FromString(TEXT("Select Fragments")))
	]
	.MenuContent()
	[
		SNew(SBox)
		[
			DropdownContent.ToSharedRef()
		]
	]
	.ComboButtonStyle(&FCoreStyle::Get().GetWidgetStyle<FComboButtonStyle>("ComboButton"))
	.ButtonStyle(&FCoreStyle::Get().GetWidgetStyle<FButtonStyle>("Button"))
	.ForegroundColor(FCoreStyle::Get().GetSlateColor("InvertedForeground"))
	.ContentPadding(FMargin(5.0f));

	FragmentSelectBox->SetContent(FragmentsButton);
#endif
}

void SMassEntitiesList::RefreshEntityData()
{
	RefreshFragmentData();
}

void SMassEntitiesList::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bAutoUpdateEntityData)
	{
		RefreshEntityData();
	}
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

void SMassEntitiesList::UpdateTreeColumns()
{
#if WITH_MASSENTITY_DEBUG

	const TIndirectArray<SHeaderRow::FColumn>& TreeColumns = TreeViewHeaderRow->GetColumns();
	bool Changed = false;

	if (TreeColumns.Num() == 0)
	{
		SHeaderRow::FColumn::FArguments ColumnArgs;
		ColumnArgs
		.ColumnId(UE::MassDebugger::EntitiesList::Private::ColumnHandle.Resolve())
		.DefaultLabel(LOCTEXT("MassEntityHandle", "Entity Handle"))
		.ToolTipText(LOCTEXT("MassEntityHandle", "Entity Handle"))
		.HAlignHeader(HAlign_Left)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Fill)
		.VAlignCell(VAlign_Fill)
		.SortMode(this, &SMassEntitiesList::GetColumnSortMode, UE::MassDebugger::EntitiesList::Private::ColumnHandle.Resolve())
		.OnSort(this, &SMassEntitiesList::OnColumnSortModeChanged)
		.FillWidth(100.f)
		.FillSized(100.f)
		.HeaderContent()
		[
			SNew(SBox)
			.Padding(FMargin(3.0f))
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("MassEntityHandle", "Entity Handle"))
			]
		];
		TreeViewHeaderRow->AddColumn(ColumnArgs);
		Changed = true;
	}

	for (int i = 0; i < Columns.Num(); i++)
	{
		FMassEntitiesListColumn& Column = Columns[i];
		int TreeViewColumnIndex = i + 1; // one extra column for the handle

		SHeaderRow::FColumn::FArguments ColumnArgs;
		ColumnArgs
		.ColumnId(Column.ColumnID)
		.DefaultLabel(FText::FromString(*Column.ColumnLabel))
		.HAlignHeader(HAlign_Left)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Fill)
		.VAlignCell(VAlign_Fill)
		.InitialSortMode(EColumnSortMode::Ascending)
		.FillWidth(100.f)
		.HeaderContent()
		[
			SNew(SBox)
			.Padding(FMargin(3.0f))
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::FromString(*Column.ColumnLabel))
			]
		];

		if (TreeColumns.Num() > TreeViewColumnIndex)
		{
			if (TreeColumns[TreeViewColumnIndex].ColumnId != Column.ColumnID)
			{
				// insert
				TreeViewHeaderRow->InsertColumn(ColumnArgs, TreeViewColumnIndex);
				Changed = true;
			}
		}
		else
		{
			TreeViewHeaderRow->AddColumn(ColumnArgs);
			Changed = true;
		}
	}

	// prune removed columns:
	for (int i = TreeColumns.Num() - 1; i >= (Columns.Num() + 1); i--)
	{
		TreeViewHeaderRow->RemoveColumn(TreeColumns[i].ColumnId);
		Changed = true;
	}

	if (Changed)
	{
		TreeView->RebuildList();
	}
#endif //WITH_MASSENTITY_DEBUG
}

EColumnSortMode::Type SMassEntitiesList::GetColumnSortMode(FName ColumnId) const
{
	if (ColumnId != UE::MassDebugger::EntitiesList::Private::ColumnHandle)
	{
		return EColumnSortMode::None;
	}

	return bSortAscending ? EColumnSortMode::Ascending : EColumnSortMode::Descending;
}

void SMassEntitiesList::OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode)
{
	if (ColumnId != UE::MassDebugger::EntitiesList::Private::ColumnHandle)
	{
		return;
	}

	if (SortPriority == EColumnSortPriority::Primary)
	{
		bSortAscending = InSortMode == EColumnSortMode::Ascending;
	}

	auto SortHandle = [bAscending = bSortAscending](const TSharedPtr<FGridRow>& A, const TSharedPtr<FGridRow>& B)
	{
		const int32 Compare = B->Entity.Index - A->Entity.Index;
		return bAscending ? Compare > 0 : Compare <= 0;
	};
	
	GridRows.StableSort(SortHandle);

	if (TreeView.IsValid())
	{
		TreeView->RequestListRefresh();
	}
}

void SMassEntitiesList::TreeView_OnGetChildren(SMassEntitiesList::EntitiesTableRowPtr InParent,
	TArray<SMassEntitiesList::EntitiesTableRowPtr>& OutChildren)
{
	return;
}

TSharedRef<ITableRow> SMassEntitiesList::TreeView_OnGenerateRow(SMassEntitiesList::EntitiesTableRowPtr Row,
	const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedRef<ITableRow> TableRow =
		SNew(SMassEntitiesList::SEntitiesTableRow, OwnerTable)
		.EntitiesTableRow(Row);

	return TableRow;
}

void SMassEntitiesList::SEntitiesTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
#if WITH_MASSENTITY_DEBUG
	TableRowPtr = InArgs._EntitiesTableRow;
	FGridRow& Row = *TableRowPtr;

	if (TableRowPtr.IsValid())
	{
		SMassEntitiesList& OwnerList = *Row.EntitiesList.Pin();
		if (OwnerList.DebuggerModel.IsValid() && !OwnerList.DebuggerModel->IsStale())
		{
			const FMassEntityManager& EntityManager = *(OwnerList.DebuggerModel->Environment->EntityManager.Pin());
			FPropertyRowGeneratorArgs GeneratorArgs;
			GeneratorArgs.bShouldShowHiddenProperties = true;
			FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

			Row.bDirty = false;
			int32 FragmentDisplayCount = OwnerList.SelectedFragmentTypes.Num();
			Row.FragmentInfo.SetNum(FragmentDisplayCount);

			for (int i = 0; i < FragmentDisplayCount; i++)
			{
				const UScriptStruct* StructType = OwnerList.SelectedFragmentTypes[i];
				if (StructType && Row.FragmentInfo[i].StructType != StructType)
				{
					Row.FragmentInfo[i].StructType = StructType;
					if (UE::Mass::IsA<FMassFragment>(StructType))
					{
						Row.FragmentInfo[i].StructData = FMassDebugger::GetFragmentData(EntityManager, OwnerList.SelectedFragmentTypes[i], Row.Entity);
						if (Row.FragmentInfo[i].StructData.IsValid())
						{
							Row.FragmentInfo[i].PropertyRowGenerator = PropertyEditorModule.CreatePropertyRowGenerator(GeneratorArgs);
							Row.FragmentInfo[i].PropertyRowGenerator->SetStructure(Row.FragmentInfo[i].StructData);
						}
					}
					else if (UE::Mass::IsA<FMassSharedFragment>(StructType))
					{
						Row.FragmentInfo[i].StructData = FMassDebugger::GetSharedFragmentData(EntityManager, OwnerList.SelectedFragmentTypes[i], Row.Entity);
						if (Row.FragmentInfo[i].StructData.IsValid())
						{
							Row.FragmentInfo[i].PropertyRowGenerator = PropertyEditorModule.CreatePropertyRowGenerator(GeneratorArgs);
							Row.FragmentInfo[i].PropertyRowGenerator->SetStructure(Row.FragmentInfo[i].StructData);
						}
					}
					else if (UE::Mass::IsA<FMassConstSharedFragment>(StructType))
					{
						Row.FragmentInfo[i].StructData = FMassDebugger::GetConstSharedFragmentData(EntityManager, OwnerList.SelectedFragmentTypes[i], Row.Entity);
						if (Row.FragmentInfo[i].StructData.IsValid())
						{
							Row.FragmentInfo[i].PropertyRowGenerator = PropertyEditorModule.CreatePropertyRowGenerator(GeneratorArgs);
							Row.FragmentInfo[i].PropertyRowGenerator->SetStructure(Row.FragmentInfo[i].StructData);
						}
					}
					else
					{
						ensureMsgf(false, TEXT("Invalid entity data type(%s)"), *StructType->GetDisplayNameText().ToString());
					}
				}
			}
		}
	}

	SetEnabled(true);
#endif //WITH_MASSENTITY_DEBUG
	Super::Construct(Super::FArguments(), InOwnerTableView);
}

TSharedRef<SWidget> SMassEntitiesList::SEntitiesTableRow::GenerateWidgetForColumn(const FName& InColumnName)
{
#if WITH_MASSENTITY_DEBUG
	if (TableRowPtr && TableRowPtr->EntitiesList.IsValid())
	{
		SMassEntitiesList& OwnerList = *TableRowPtr->EntitiesList.Pin();
		if (!OwnerList.DebuggerModel->Environment.IsValid()
			|| !OwnerList.DebuggerModel->Environment->GetEntityManager().IsValid())
		{
			return SNullWidget::NullWidget;
		}

		if (InColumnName == FName(TEXT("EntityHandle")))
		{
			FMassEntityHandle Entity = TableRowPtr->Entity;
			TWeakPtr<const FMassEntityManager> WeakEntityManager = OwnerList.DebuggerModel->Environment->GetEntityManager().ToWeakPtr();

			return SNew(SButton)
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Fill)
			.Text(FText::FromString(Entity.DebugGetDescription()))
			.OnClicked_Lambda([WeakEntityManager, Entity]()
			{
				if (Entity.IsValid() && WeakEntityManager.IsValid())
				{
					FMassDebugger::SelectEntity(*WeakEntityManager.Pin(), Entity);
				}
				return FReply::Handled();
			});
		}

		int32* ColumnIndex = OwnerList.ColumnIndexByID.Find(InColumnName);

		if (ColumnIndex && *ColumnIndex >= 0 && *ColumnIndex < OwnerList.Columns.Num())
		{
			FMassEntitiesListColumn& Column = OwnerList.Columns[*ColumnIndex];

			if (Column.StructType)
			{
				for (SMassEntitiesList::FGridRow::FFragmentInfo& Info : TableRowPtr->FragmentInfo)
				{
					if (Info.StructType == Column.StructType)
					{
						if (Column.Property)
						{
							return GenerateDataWidget(Column.Property, Info);
						}
						else
						{
							return GenerateBreakpointWidget(Info);
						}
					}
				}
			}
		}
	}
#endif //WITH_MASSENTITY_DEBUG
	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> SMassEntitiesList::SEntitiesTableRow::GenerateBreakpointWidget(SMassEntitiesList::FGridRow::FFragmentInfo& Info)
{
#if WITH_MASSENTITY_DEBUG
	FMassEntityHandle Entity = TableRowPtr->Entity;
	SMassEntitiesList& OwnerList = *TableRowPtr->EntitiesList.Pin();
	TWeakPtr<FMassDebuggerModel> WeakModel = OwnerList.DebuggerModel.ToWeakPtr();
	if (!OwnerList.DebuggerModel->Environment.IsValid()
		|| !OwnerList.DebuggerModel->Environment->GetEntityManager().IsValid())
	{
		return SNullWidget::NullWidget;
	}
	TWeakPtr<const FMassEntityManager> WeakEntityManager = OwnerList.DebuggerModel->Environment->GetEntityManager().ToWeakPtr();

	const UScriptStruct* FragmentType = Info.StructType;

	return SNew(SVerticalBox)
	+ SVerticalBox::Slot()
	.AutoHeight()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(4.0f)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "FlatButton")
			.ContentPadding(4.0f)
			.ToolTipText(LOCTEXT("SetWriteBreakpoint", "Set Write Breakpoint"))
			.OnClicked_Lambda([Entity, FragmentType, WeakModel]()
			{
				TSharedPtr<FMassDebuggerModel> Model = WeakModel.Pin();
				if (Model.IsValid() && !Model->IsStale())
				{
					FMassDebugger::SetFragmentWriteBreakpoint(*Model->Environment->EntityManager.Pin(), FragmentType, Entity);
				}
				return FReply::Handled();
			})
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("GenericStop"))
				.DesiredSizeOverride(FVector2D(16, 16))
			]
		]
	];
#else //WITH_MASSENTITY_DEBUG
	return SNullWidget::NullWidget;
#endif //WITH_MASSENTITY_DEBUG
}

TSharedRef<SWidget> SMassEntitiesList::SEntitiesTableRow::GenerateDataWidget(const FProperty* Property, SMassEntitiesList::FGridRow::FFragmentInfo& Info)
{
#if WITH_MASSENTITY_DEBUG
	if (Info.PropertyRowGenerator.IsValid())
	{
		SMassEntitiesList& OwnerList = *TableRowPtr->EntitiesList.Pin();

		if (!Property)
		{
			return SNullWidget::NullWidget;
		}
		
		OwnerList.NodesToSearch.Reserve(100);
		OwnerList.NodesToSearch = Info.PropertyRowGenerator->GetRootTreeNodes();

		// some of the properties we want to find might be nested in categories so we need to search the tree
		TSharedPtr<IPropertyHandle> PropertyHandle = FindPropertyHandle(Property, OwnerList.NodesToSearch);
		OwnerList.NodesToSearch.Reset();
		if (PropertyHandle.IsValid())
		{
			TSharedPtr<SHorizontalBox> HBox = SNew(SHorizontalBox);
			// Found a top-level property on the fragment
			TSharedPtr<SVerticalBox> VBox = SNew(SVerticalBox);
			HBox->AddSlot()
			.HAlign(EHorizontalAlignment::HAlign_Center)
			.VAlign(EVerticalAlignment::VAlign_Top)
			.AutoWidth()
			[
				VBox.ToSharedRef()
			];
			OwnerList.AddPropertyRecursive(HBox, VBox, PropertyHandle);

			return HBox.ToSharedRef();
		}
	}
#endif //WITH_MASSENTITY_DEBUG
	return SNullWidget::NullWidget;
}

#undef LOCTEXT_NAMESPACE