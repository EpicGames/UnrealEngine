// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"
#include "MassDebugger.h"
#include "MassDebuggerModel.h"

class IDetailTreeNode;
class IPropertyHandle;
class IPropertyRowGenerator;
class FStructOnScope;
class SGridPanel;
class SBox;
class SScrollBar;
class SHorizontalBox;
class SVerticalBox;

class SMassEntitiesList : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMassEntitiesList){}	
		SLATE_ARGUMENT(TArray<FMassEntityHandle>, Entities)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<FMassDebuggerModel> InDebuggerModel);
	void SetEntities(const TArray<FMassEntityHandle>& InEntities);
	void RefreshEntityData();

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

protected:

	struct FMassEntitiesListColumn
	{
		const UScriptStruct* StructType = nullptr;
		const FProperty* Property = nullptr;
		FString ColumnLabel;
		FName ColumnID;
	};

	struct FGridRow
	{
		struct FFragmentInfo
		{
			const UScriptStruct* StructType = nullptr;
			TSharedPtr<IPropertyRowGenerator> PropertyRowGenerator;

			// this is a snapshot copy of the actual mass fragment data
			TSharedPtr<FStructOnScope> StructData;
		};
		
		TArray<FFragmentInfo> FragmentInfo;
		FMassEntityHandle Entity;
		TWeakPtr<SMassEntitiesList> EntitiesList;
		bool bDirty = true;
	};



public:

	using EntitiesTableRowPtr = TSharedPtr<FGridRow>;

	class SEntitiesTableRow : public SMultiColumnTableRow<EntitiesTableRowPtr>
	{
		using Super = SMultiColumnTableRow<EntitiesTableRowPtr>;
	public:
		SLATE_BEGIN_ARGS(SEntitiesTableRow) {}
			SLATE_ARGUMENT(EntitiesTableRowPtr, EntitiesTableRow)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);

		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override;

	private:
		EntitiesTableRowPtr TableRowPtr;
		TSharedRef<SWidget> GenerateBreakpointWidget(FGridRow::FFragmentInfo& Info);
		TSharedRef<SWidget> GenerateDataWidget(const FProperty* Property, FGridRow::FFragmentInfo& Info);
	};

	TSharedPtr<SBox> GetFragmentSelectBox()
	{
		return FragmentSelectBox;
	}

	void AutoUpdateEntityData(bool bEnable)
	{
		bAutoUpdateEntityData = bEnable;
	}

private:
	TArray<EntitiesTableRowPtr> GridRows;
	TArray<FMassEntitiesListColumn> Columns;
	TMap<FName, int32> ColumnIndexByID;

	void BuildGrid();
	void PopulateGridColumns();
	void RefreshFragmentData();

	void AddPropertyRecursive(TSharedPtr<SHorizontalBox> HBox, TSharedPtr<SVerticalBox> VBox, TSharedPtr<IPropertyHandle> Prop, bool bShowName = false);
	TArray<FName> AvailableFragmentNames;
	TArray<FName> SelectedFragmentNames;
	TArray<const UScriptStruct*> SelectedFragmentTypes;
	FReply OnClearAllSelectedFragmentsClicked();

	TSharedPtr<SBox> FragmentSelectBox;

	bool bAutoUpdateEntityData = false;

	void OnFragmentCheckStateChanged(ECheckBoxState NewState, FName FragmentName);
	ECheckBoxState GetFragmentCheckState(FName FragmentName) const;
	void CreateFragmentSelectDropdown();
	TSharedPtr<FMassDebuggerModel> DebuggerModel;
	void UpdateTreeColumns();
	EColumnSortMode::Type GetColumnSortMode(FName ColumnId) const;
	void OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode);
	bool bSortAscending = true;

	void TreeView_OnGetChildren(SMassEntitiesList::EntitiesTableRowPtr InParent, TArray<SMassEntitiesList::EntitiesTableRowPtr>& OutChildren);
	TSharedRef<ITableRow> TreeView_OnGenerateRow(EntitiesTableRowPtr RowPtr, const TSharedRef<STableViewBase>& OwnerTable);

	TSharedPtr<STreeView<EntitiesTableRowPtr>> TreeView;
	TSharedPtr<SHeaderRow> TreeViewHeaderRow;
	TArray<TSharedRef<IDetailTreeNode>> NodesToSearch;
};
