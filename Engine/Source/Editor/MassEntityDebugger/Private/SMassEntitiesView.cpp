// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMassEntitiesView.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSlider.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/SBoxPanel.h"
#include "MassDebugger.h"
#include "MassDebuggerModel.h"
#include "MassEntityQuery.h"
#include "SMassEntitiesList.h"

#define LOCTEXT_NAMESPACE "SMassDebugger"

//----------------------------------------------------------------------//
// SMassEntitiesView
//----------------------------------------------------------------------//
void SMassEntitiesView::Construct(const FArguments& InArgs, TSharedRef<FMassDebuggerModel> InDebuggerModel, uint32 InEntitiesViewIndex)
{
#if WITH_MASSENTITY_DEBUG
	Initialize(InDebuggerModel);

	EntitiesList = SNew(SMassEntitiesList, InDebuggerModel);
	ListLabel = SNew(STextBlock);

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(5)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				ListLabel.ToSharedRef()
			]
			+ SHorizontalBox::Slot()
			.Padding(5)
			.AutoWidth()
			[
				EntitiesList->GetFragmentSelectBox().ToSharedRef()
			]
			+ SHorizontalBox::Slot()
			.Padding(5)
			.AutoWidth()
			[
				SNew(SButton)
				.VAlign(VAlign_Center)
				.Text(LOCTEXT("ShowAllEntities", "ShowAllEntities"))
				.OnClicked(FOnClicked::CreateSP(this, &SMassEntitiesView::ShowAllEntities))
			]
			+ SHorizontalBox::Slot()
			.Padding(5)
			.AutoWidth()
			[
				SNew(SButton)
				.VAlign(VAlign_Center)
				.Text(LOCTEXT("RefreshEntityList", "Refresh Entities"))
				.OnClicked(FOnClicked::CreateSP(this, &SMassEntitiesView::RefreshEntityList))
			]
			+ SHorizontalBox::Slot()
			.Padding(5)
			.AutoWidth()
			[
				SNew(SButton)
				.VAlign(VAlign_Center)
				.Text(LOCTEXT("RefreshEntityData", "Refresh Entity Data"))
				.OnClicked(FOnClicked::CreateSP(this, &SMassEntitiesView::RefreshEntityData))
			]
			+ SHorizontalBox::Slot()
			.Padding(5)
			.AutoWidth()
			[

				SAssignNew(AutoUpdateCheckbox, SCheckBox)
				.OnCheckStateChanged(this, &SMassEntitiesView::OnAutoUpdateChanged)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("AutoUpdateEntityData", "Auto Update Entity Data"))
				]
			]

		]
		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			EntitiesList.ToSharedRef()
		]
	];

	InDebuggerModel->RegisterEntitiesView(SharedThis(this), InEntitiesViewIndex);

#else
	ChildSlot
	[
		SNew(STextBlock)
		.Text(LOCTEXT("MassEntityDebuggingNotEnabled", "Mass Entity Debugging Not Enabled for this configuration"))
	];
#endif
}

void SMassEntitiesView::OnAutoUpdateChanged(ECheckBoxState NewState)
{
	if (EntitiesList.IsValid())
	{
		EntitiesList->AutoUpdateEntityData(NewState == ECheckBoxState::Checked);
	}
}

void SMassEntitiesView::ShowEntities(FMassArchetypeHandle InArchetypeHandle)
{
#if WITH_MASSENTITY_DEBUG
	ArchetypeHandle = InArchetypeHandle;
	ShowEntitiesFrom = EShowEntitiesFrom::Archetype;
	RefreshEntityList();
#endif
}

void SMassEntitiesView::ShowEntities(const TArray<FMassEntityHandle>& InEntities)
{
#if WITH_MASSENTITY_DEBUG
	ArchetypeHandle = FMassArchetypeHandle();
	ShowEntitiesFrom = EShowEntitiesFrom::List;
	if (EntitiesList.IsValid())
	{
		EntitiesList->SetEntities(InEntities);
	}
#endif
}

void SMassEntitiesView::ShowEntities(FMassEntityQuery& InQuery)
{
#if WITH_MASSENTITY_DEBUG
	Query = InQuery;
	ShowEntitiesFrom = EShowEntitiesFrom::Query;
	RefreshEntityList();
#endif
}

void SMassEntitiesView::ShowEntities(TConstArrayView<FMassEntityQuery*> InQueries)
{
#if WITH_MASSENTITY_DEBUG
	Queries = InQueries;
	ShowEntitiesFrom = EShowEntitiesFrom::QueryList;
	RefreshEntityList();
#endif
}

FReply SMassEntitiesView::ShowAllEntities()
{
#if WITH_MASSENTITY_DEBUG
	ArchetypeHandle = FMassArchetypeHandle();
	ShowEntitiesFrom = EShowEntitiesFrom::All;
	RefreshEntityList();
#endif
	return FReply::Handled();
}

void SMassEntitiesView::ClearEntities()
{
#if WITH_MASSENTITY_DEBUG
	TempEntityList.Reset();
	ShowEntitiesFrom = EShowEntitiesFrom::List;
	RefreshEntityList();
#endif
}

FReply SMassEntitiesView::RefreshEntityList()
{
#if WITH_MASSENTITY_DEBUG
	if (!DebuggerModel.IsValid()
		|| !DebuggerModel->Environment.IsValid()
		|| !DebuggerModel->Environment->EntityManager.IsValid()
		|| DebuggerModel->IsStale())
	{
		TempEntityList.Reset();
		if (EntitiesList.IsValid())
		{
			EntitiesList->SetEntities(TempEntityList);
		}
		return FReply::Handled();
	}

	const FMassEntityManager& EntityManager = *DebuggerModel->Environment->EntityManager.Pin().Get();

	switch (ShowEntitiesFrom)
	{
	case EShowEntitiesFrom::All:
		{
			TArray<FMassArchetypeHandle> Archetypes = FMassDebugger::GetAllArchetypes(EntityManager);
			TempEntityList.Reset();
			for (FMassArchetypeHandle& ArchHandle : Archetypes)
			{
				TempEntityList.Append(FMassDebugger::GetEntitiesOfArchetype(ArchHandle));
			}
			EntitiesList->SetEntities(TempEntityList);
			break;
		}
	case EShowEntitiesFrom::Archetype:
		TempEntityList.Reset();
		TempEntityList.Append(FMassDebugger::GetEntitiesOfArchetype(ArchetypeHandle));
		EntitiesList->SetEntities(TempEntityList);
		break;
	case EShowEntitiesFrom::Query:
		TempEntityList.Reset();
		TempEntityList.Append(FMassDebugger::GetEntitiesMatchingQuery(EntityManager, Query));
		EntitiesList->SetEntities(TempEntityList);
		break;
	case EShowEntitiesFrom::QueryList:
		{
			TempEntityList.Reset();
			for (FMassEntityQuery* QueryInList : Queries)
			{
				if (QueryInList)
				{
					TempEntityList.Append(FMassDebugger::GetEntitiesMatchingQuery(EntityManager, *QueryInList));
				}
			}
			EntitiesList->SetEntities(TempEntityList);
			break;
		}
	case EShowEntitiesFrom::List:
	default:
		break;
	}
#endif
	return FReply::Handled();
}

FReply SMassEntitiesView::RefreshEntityData()
{
#if WITH_MASSENTITY_DEBUG
	if (EntitiesList.IsValid())
	{
		EntitiesList->RefreshEntityData();
	}
#endif
	return FReply::Handled();
}

void SMassEntitiesView::OnRefresh()
{
#if WITH_MASSENTITY_DEBUG
	RefreshEntityList();
#endif
}

#undef LOCTEXT_NAMESPACE

