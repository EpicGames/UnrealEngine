// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "SMassDebuggerViewBase.h"
#include "Widgets/Views/SListView.h"
#include "Mass/EntityHandle.h"
#include "MassArchetypeTypes.h"
#include "MassEntityQuery.h"

class SMassEntitiesList;
class SCheckBox;

class SMassEntitiesView : public SMassDebuggerViewBase
{
public:
	SLATE_BEGIN_ARGS(SMassEntitiesView)
	{
	}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<FMassDebuggerModel> InDebuggerModel, uint32 InEntitiesViewIndex);

	void ShowEntities(FMassArchetypeHandle InArchetypeHandle);
	void ShowEntities(const TArray<FMassEntityHandle>& InEntities);
	void ShowEntities(FMassEntityQuery& InQuery);
	void ShowEntities(TConstArrayView<FMassEntityQuery*> InQueries);
	void ClearEntities();

protected:
	enum class EShowEntitiesFrom
	{
		List,
		Archetype,
		Query,
		QueryList,
		All
	};
	FReply ShowAllEntities();
	FReply RefreshEntityList();
	FReply RefreshEntityData();

	void OnAutoUpdateChanged(ECheckBoxState NewState);
	float GetUpdateInterval() const;
	void OnUpdateIntervalChanged(float NewValue);

	virtual void OnRefresh() override;

	/** Unused pure virtual function */
	virtual void OnProcessorsSelected(TConstArrayView<TSharedPtr<FMassDebuggerProcessorData>> SelectedProcessors, ESelectInfo::Type SelectInfo) override
	{
		//@todo: highlight entities affected by the selected processor
	}

	/** Unused pure virtual function */
	virtual void OnArchetypesSelected(TConstArrayView<TSharedPtr<FMassDebuggerArchetypeData>> SelectedArchetypes, ESelectInfo::Type SelectInfo) override
	{
		//@todo: highlight entities of the selected archetype
	}

	EShowEntitiesFrom ShowEntitiesFrom;
	TSharedPtr<SMassEntitiesList> EntitiesList;
	FMassArchetypeHandle ArchetypeHandle;
	TSharedPtr<STextBlock> ListLabel;
	TArray<FMassEntityHandle> TempEntityList;
	uint32 EntitiesViewIndex;
	FMassEntityQuery Query;
	TConstArrayView<FMassEntityQuery*> Queries;
	TSharedPtr<SCheckBox> AutoUpdateCheckbox;
};
