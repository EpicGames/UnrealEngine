// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMassFragmentsView.h"

#include "MassDebuggerModel.h"
#include "MassEntityManager.h"         
#include "MassArchetypeData.h"         
#include "MassProcessingTypes.h"       
#include "MassProcessor.h"             

#include "Styling/AppStyle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/Views/TableViewMetadata.h"
#include "Misc/TextFilter.h"

#define LOCTEXT_NAMESPACE "SMassDebugger"

//----------------------------------------------------------------------//
// SFragmentsView
//----------------------------------------------------------------------//
namespace UE::MassDebugger::FragmentsView
{
	namespace Private
	{
		static const FLazyName ColumnName = TEXT("Name");
		static const FLazyName ColumnNumArchetypes = TEXT("Archetypes");
		static const FLazyName ColumnNumEntities = TEXT("Entities");
	}

void SFragmentsView::Construct(const FArguments& InArgs, TSharedRef<FMassDebuggerModel> InDebuggerModel)
{ 
#if WITH_MASSENTITY_DEBUG
	Initialize(InDebuggerModel);
	TSharedRef<SHeaderRow> HeaderRowWidget =
		SNew(SHeaderRow)
		+ SHeaderRow::Column(Private::ColumnName)
		.DefaultLabel(LOCTEXT("FragmentColumnName", "Fragment Type"))
		.FillWidth(0.5f)
		.SortMode(this, &SFragmentsView::GetColumnSortMode, Private::ColumnName.Resolve())
		.OnSort(this, &SFragmentsView::OnColumnSortModeChanged)
		+ SHeaderRow::Column(Private::ColumnNumArchetypes)
		.DefaultLabel(LOCTEXT("FragmentColumnArchetypes", "Archetypes"))
		.FillWidth(0.25f)
		.SortMode(this, &SFragmentsView::GetColumnSortMode, Private::ColumnNumArchetypes.Resolve())
		.OnSort(this, &SFragmentsView::OnColumnSortModeChanged)
		+ SHeaderRow::Column(Private::ColumnNumEntities)
		.DefaultLabel(LOCTEXT("FragmentColumnEntities", "Entities"))
		.FillWidth(0.25f)
		.SortMode(this, &SFragmentsView::GetColumnSortMode, Private::ColumnNumEntities.Resolve())
		.OnSort(this, &SFragmentsView::OnColumnSortModeChanged);

	SAssignNew(FragmentsList, SListView<FFragmentsTableRowPtr>)
		.ListItemsSource(&FragmentListItemsSource)
		.OnGenerateRow(this, &SFragmentsView::OnGenerateFragmentRow)
		.HeaderRow(HeaderRowWidget);

	ChildSlot
		[
			FragmentsList.ToSharedRef()
		];

	RefreshFragmentList();

#else // WITH_MASSENTITY_DEBUG
	ChildSlot
		[
			SNew(STextBlock)
				.Text(LOCTEXT("MassEntityDebuggingNotEnabled", "Mass Entity Debugging Not Enabled for this configuration"))
				.Justification(ETextJustify::Center)
		];
#endif // WITH_MASSENTITY_DEBUG
}

TSharedRef<ITableRow> SFragmentsView::OnGenerateFragmentRow(FFragmentsTableRowPtr InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SFragmentsTableRow, OwnerTable, InItem);
}

EColumnSortMode::Type SFragmentsView::GetColumnSortMode(FName ColumnId) const
{
	if (SortByColumnId != ColumnId)
	{
		return EColumnSortMode::None;
	}

	return bSortAscending ? EColumnSortMode::Ascending : EColumnSortMode::Descending;
}

void SFragmentsView::OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode)
{
	if (SortPriority == EColumnSortPriority::Primary)
	{
		SortByColumnId = ColumnId;
		bSortAscending = InSortMode == EColumnSortMode::Ascending ? true : false;
	}
	RefreshFragmentList();
}

void SFragmentsView::OnRefresh()
{
#if WITH_MASSENTITY_DEBUG
	RefreshFragmentList();
#endif // WITH_MASSENTITY_DEBUG
}

void SFragmentsView::RefreshFragmentList()
{
	FragmentListItemsSource.Reset();
#if WITH_MASSENTITY_DEBUG
	if (!DebuggerModel.IsValid())
	{
		return;
	}

	FragmentListItemsSource.Reserve(DebuggerModel->CachedFragmentData.Num());

	for (TSharedPtr<FMassDebuggerFragmentData>& FragmentData : DebuggerModel->CachedFragmentData)
	{
		// filter out unused fragments
		// @todo: add checkbox to "Show Unused Fragments"
		if (FragmentData->Archetypes.Num() > 0)
		{
			FragmentListItemsSource.Add(FragmentData);
		}
	}

	if (SortByColumnId == Private::ColumnName)
	{
		auto SortName = [bAscending = bSortAscending](const TSharedPtr<FMassDebuggerFragmentData>& A, const TSharedPtr<FMassDebuggerFragmentData>& B)
		{
			const int32 Compare = A->Name.CompareTo(B->Name);
			return bAscending ? Compare < 0 : Compare >= 0;
		};
		FragmentListItemsSource.StableSort(SortName);
	}
	else if (SortByColumnId == Private::ColumnNumArchetypes)
	{
		auto SortArchetypes = [bAscending = bSortAscending](const TSharedPtr<FMassDebuggerFragmentData>& A, const TSharedPtr<FMassDebuggerFragmentData>& B)
		{
			return bAscending ? A->Archetypes.Num() < B->Archetypes.Num() : A->Archetypes.Num() > B->Archetypes.Num();
		};
		FragmentListItemsSource.StableSort(SortArchetypes);
	}
	else if (SortByColumnId == Private::ColumnNumEntities)
	{
		auto SortEntities = [bAscending = bSortAscending](const TSharedPtr<FMassDebuggerFragmentData>& A, const TSharedPtr<FMassDebuggerFragmentData>& B)
		{
			return bAscending ? A->NumEntities < B->NumEntities : A->NumEntities > B->NumEntities;
		};
		FragmentListItemsSource.StableSort(SortEntities);
	}

	if (FragmentsList.IsValid())
	{
		FragmentsList->RequestListRefresh();
	}
#endif // WITH_MASSENTITY_DEBUG
}

//----------------------------------------------------------------------//
// SFragmentsTableRow - Represents a single row in the list
//----------------------------------------------------------------------//
void SFragmentsTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, FFragmentsTableRowPtr InFragmentDataPtr)
{
#if WITH_MASSENTITY_DEBUG

	FragmentDataPtr = InFragmentDataPtr;

	SMultiColumnTableRow<FFragmentsTableRowPtr>::Construct(
		SMultiColumnTableRow<FFragmentsTableRowPtr>::FArguments()
		.ShowSelection(false)
		, InOwnerTableView);

#endif //WITH_MASSENTITY_DEBUG
}

TSharedRef<SWidget> SFragmentsTableRow::GenerateWidgetForColumn(const FName& InColumnName)
{
	TSharedRef<SWidget> CellWidget = SNullWidget::NullWidget;

#if WITH_MASSENTITY_DEBUG

	if (!FragmentDataPtr.IsValid())
	{
		return CellWidget;
	}

	if (InColumnName == Private::ColumnName)
	{
		SAssignNew(CellWidget, STextBlock)
			.Text(FragmentDataPtr->Name)
			.ToolTipText(FragmentDataPtr->Fragment.IsValid() ? FText::FromString(FragmentDataPtr->Fragment.Pin()->GetPathName()) : LOCTEXT("InvalidFragmentPath", "Invalid Fragment Struct"));
	}
	else if (InColumnName == Private::ColumnNumArchetypes)
	{
		SAssignNew(CellWidget, STextBlock)
			.Text(FText::AsNumber(FragmentDataPtr->Archetypes.Num()))
			.Justification(ETextJustify::Right);
	}
	else if (InColumnName == Private::ColumnNumEntities)
	{
		SAssignNew(CellWidget, STextBlock)
			.Text(FText::AsNumber(FragmentDataPtr->NumEntities))
			.Justification(ETextJustify::Right);
	}

#endif //WITH_MASSENTITY_DEBUG

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
			.Padding(FMargin(4.f, 0.f))
			.VAlign(VAlign_Center)
			[
				CellWidget
			];
}

} // UE::MassDebugger::FragmentsView

#undef LOCTEXT_NAMESPACE