// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "SMassDebuggerViewBase.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Templates/SharedPointer.h"

struct FMassDebuggerModel;
struct FMassDebuggerFragmentData;
class SHeaderRow;
class STableViewBase;
class UMassProcessor;
class UScriptStruct;
struct FMassArchetypeHandle;


namespace UE::MassDebugger::FragmentsView
{

using FFragmentsTableRowPtr = TSharedPtr<FMassDebuggerFragmentData>;

class SFragmentsTableRow : public SMultiColumnTableRow<FFragmentsTableRowPtr>
{
	using Super = SMultiColumnTableRow<FFragmentsTableRowPtr>;
public:

	SLATE_BEGIN_ARGS(SFragmentsTableRow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, FFragmentsTableRowPtr InFragmentDataPtr);
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override;

private:
	FFragmentsTableRowPtr FragmentDataPtr;
};


class SFragmentsView : public SMassDebuggerViewBase
{
public:
	SLATE_BEGIN_ARGS(SFragmentsView)
		{}
	SLATE_END_ARGS()
	void Construct(const FArguments& InArgs, TSharedRef<FMassDebuggerModel> InDebuggerModel);

	virtual void OnRefresh() override;
	virtual void OnProcessorsSelected(TConstArrayView<TSharedPtr<FMassDebuggerProcessorData>> SelectedProcessors, ESelectInfo::Type SelectInfo) override
	{}
	virtual void OnArchetypesSelected(TConstArrayView<TSharedPtr<FMassDebuggerArchetypeData>> SelectedArchetypes, ESelectInfo::Type SelectInfo) override
	{}

protected:
	TSharedRef<ITableRow> OnGenerateFragmentRow(FFragmentsTableRowPtr InItem, const TSharedRef<STableViewBase>& OwnerTable);
	void RefreshFragmentList();

	EColumnSortMode::Type GetColumnSortMode(FName ColumnId) const;
	void OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode);

private:
	TSharedPtr<SListView<FFragmentsTableRowPtr>> FragmentsList;
	TArray<FFragmentsTableRowPtr> FragmentListItemsSource;
	FName SortByColumnId;
	bool bSortAscending = true;
};

} // UE::MassDebugger::FragmentsView