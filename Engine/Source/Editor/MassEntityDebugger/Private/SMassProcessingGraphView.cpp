// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMassProcessingGraphView.h"
#include "MassDebuggerModel.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Styling/SlateBrush.h"
#include "MassDebuggerStyle.h"
#include "Widgets/Views/STreeView.h"
#include "Styling/StyleColors.h"

#define LOCTEXT_NAMESPACE "SMassDebugger"


//----------------------------------------------------------------------//
// FMassDebuggerProcessingGraphNodeTreeItem
//----------------------------------------------------------------------//
FMassDebuggerProcessingGraphNodeTreeItem::FMassDebuggerProcessingGraphNodeTreeItem(const FMassDebuggerProcessingGraphNode& InNode)
	: Node(InNode)
{

}

//----------------------------------------------------------------------//
// SMassProcessingGraphTableRow
//----------------------------------------------------------------------//
using FMassDebuggerProcessingGraphNodeTreeItemPtr = TSharedPtr<FMassDebuggerProcessingGraphNodeTreeItem, ESPMode::ThreadSafe>;

class SMassProcessingGraphTableRow : public STableRow<FMassDebuggerProcessingGraphNodeTreeItemPtr>
{
public:
	SLATE_BEGIN_ARGS(SMassProcessingGraphTableRow) { }
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedPtr<STableViewBase>& InOwnerTableView, const FMassDebuggerProcessingGraphNodeTreeItemPtr InEntryItem)
	{
		Item = InEntryItem;

		STableRow<FMassDebuggerProcessingGraphNodeTreeItemPtr>::ConstructInternal(STableRow::FArguments()
			.Padding(5.0f)
			, InOwnerTableView.ToSharedRef());

		ChildSlot
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Left)
				.AutoWidth()
				[
					SNew(SExpanderArrow, SharedThis(this))
					.ShouldDrawWires(true)
					.IndentAmount(32)
					.BaseIndentLevel(0)
				]

				+ SHorizontalBox::Slot()
				[
					SNew(SBox)
					.Padding(4, 2)
					[
						
						SNew(STextBlock)
						.Text(Item->Node.GetLabel())
						.ColorAndOpacity_Lambda([this]()
							{
								const FLinearColor Foreground = FStyleColors::Foreground.GetSpecifiedColor();
							
								switch (Item->Node.GraphNodeSelection)
								{
									case EMassDebuggerProcessingGraphNodeSelection::None:
										return Item->Node.ProcessorData->bIsActive
											? FLinearColor::White
											: FLinearColor::Gray;
									case EMassDebuggerProcessingGraphNodeSelection::WaitFor:
										return FMath::Lerp(Foreground, FLinearColor::Green, 0.75f);
									case EMassDebuggerProcessingGraphNodeSelection::Block:
										return FMath::Lerp(Foreground, FLinearColor::Red, 0.75f);
									default:
										return FLinearColor::White;
								}
							})
					]
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Left)
				.AutoWidth()
				[
					SNew(SBox)
				]
			];
	}

	FMassDebuggerProcessingGraphNodeTreeItemPtr Item;
};

//----------------------------------------------------------------------//
// SMassProcessingGraphView
//----------------------------------------------------------------------//
void SMassProcessingGraphView::Construct(const FArguments& InArgs, TSharedRef<FMassDebuggerModel> InDebuggerModel)
{
	Initialize(InDebuggerModel);

	OffsetPerLevel = InArgs._OffsetPerLevel.Get();

	ItemsBox = SNew(SVerticalBox);

	ChildSlot
	[
		SNew(SScrollBox)
		.Orientation(Orient_Horizontal)
		+ SScrollBox::Slot()
		[
			SAssignNew(GraphNodesTree, STreeView<TSharedPtr<FMassDebuggerProcessingGraphNodeTreeItem>>)
			.SelectionMode(ESelectionMode::Single)
			.TreeItemsSource(&RootNodes)
			.TreeViewStyle(&FAppStyle::Get().GetWidgetStyle<FTableViewStyle>("PropertyTable.InViewport.ListView"))
			.OnGenerateRow_Lambda([this](TSharedPtr<FMassDebuggerProcessingGraphNodeTreeItem> Item, const TSharedPtr<STableViewBase>& OwnerTable)
				{
					return SNew(SMassProcessingGraphTableRow, OwnerTable, Item);
				})
			.OnGetChildren_Lambda([](TSharedPtr<FMassDebuggerProcessingGraphNodeTreeItem> InItem, TArray<TSharedPtr<FMassDebuggerProcessingGraphNodeTreeItem>>& OutChildren)
				{
					if (InItem->ChildItems.Num())
					{
						OutChildren.Append(InItem->ChildItems);
					}
				})
				.OnSelectionChanged(this, &SMassProcessingGraphView::HandleSelectionChanged)
		]
	];
}

void SMassProcessingGraphView::Display(TSharedPtr<FMassDebuggerProcessingGraph> InProcessingGraphData)
{
	ItemsBox->ClearChildren();
	RootNodes.Reset();
	AllNodes.Reset();

	if (!InProcessingGraphData || InProcessingGraphData->GraphNodes.Num() == 0)
	{
		GraphNodesTree->RebuildList();
		return;
	}

	const FSlateBrush* Brush = FMassDebuggerStyle::GetBrush("MassDebug.Fragment");

	TArray<int32> Offset;
	Offset.Reserve(InProcessingGraphData->GraphNodes.Num());
	AllNodes.Reserve(InProcessingGraphData->GraphNodes.Num());

	for (const FMassDebuggerProcessingGraphNode& Node : InProcessingGraphData->GraphNodes)
	{
		const int32 ThisNodesIndex = AllNodes.Num();
		TSharedPtr<FMassDebuggerProcessingGraphNodeTreeItem>& SharedNode = AllNodes.Add_GetRef(MakeShareable(new FMassDebuggerProcessingGraphNodeTreeItem(Node)));

		int32 MyOffset = INDEX_NONE;
		if (Node.WaitForNodes.Num())
		{
			int32 LastDependencyIndex = INDEX_NONE;
			for (const int32 DependencyIndex : Node.WaitForNodes)
			{
				AllNodes[DependencyIndex]->Node.BlockNodes.Add(ThisNodesIndex);

				// note that we're accepting == offsets as well to attach to the last found item
				if (MyOffset <= Offset[DependencyIndex])
				{
					MyOffset = Offset[DependencyIndex];
					LastDependencyIndex = DependencyIndex;
				}
			}
			check(LastDependencyIndex != INDEX_NONE);
			AllNodes[LastDependencyIndex]->ChildItems.Add(SharedNode);
		}
		else
		{
			RootNodes.Add(SharedNode);
		}
		MyOffset++;
		Offset.Add(MyOffset);
	}

	GraphNodesTree->RebuildList();

	for (TSharedPtr<FMassDebuggerProcessingGraphNodeTreeItem>& Node : AllNodes)
	{
		GraphNodesTree->SetItemExpansion(Node, true);
	}
}

void SMassProcessingGraphView::HandleSelectionChanged(TSharedPtr<FMassDebuggerProcessingGraphNodeTreeItem> InNode, ESelectInfo::Type InSelectInfo)
{
	if (InSelectInfo == ESelectInfo::Direct)
	{
		return;
	}

	ClearSelection();

	if (InNode)
	{
		DebuggerModel->SelectProcessor(InNode->Node.ProcessorData);
		MarkDependencies(InNode->Node);
	}
}

void SMassProcessingGraphView::OnRefresh()
{
	// nothing to do here, but we need to implement this function since it's pure virtual in the parent class.
}

void SMassProcessingGraphView::OnProcessorsSelected(TConstArrayView<TSharedPtr<FMassDebuggerProcessorData>> SelectedProcessors, ESelectInfo::Type)
{
	ClearSelection();
	if (SelectedProcessors.Num() && SelectedProcessors[0].IsValid())
	{
		const uint32 ProcessorHash = SelectedProcessors[0]->ProcessorHash;
		TSharedPtr<FMassDebuggerProcessingGraphNodeTreeItem>* MatchingItem = AllNodes.FindByPredicate([ProcessorHash](const TSharedPtr<FMassDebuggerProcessingGraphNodeTreeItem>& Element) -> bool
		{
			return Element.IsValid() && Element->Node.ProcessorData && Element->Node.ProcessorData->ProcessorHash == ProcessorHash;
		});

		if (MatchingItem)
		{
			GraphNodesTree->SetSelection(*MatchingItem);
			GraphNodesTree->RequestScrollIntoView(*MatchingItem);
			MarkDependencies((*MatchingItem)->Node);
			GraphNodesTree->RebuildList();
		}
	}
}

void SMassProcessingGraphView::OnArchetypesSelected(TConstArrayView<TSharedPtr<FMassDebuggerArchetypeData>> SelectedArchetypes, ESelectInfo::Type)
{
	if (DebuggerModel)
	{
		OnProcessorsSelected(DebuggerModel->SelectedProcessors, ESelectInfo::Direct);
	}
}

void SMassProcessingGraphView::ClearSelection()
{
	GraphNodesTree->ClearSelection();

	for (TSharedPtr<FMassDebuggerProcessingGraphNodeTreeItem>& Node : AllNodes)
	{
		Node->Node.GraphNodeSelection = EMassDebuggerProcessingGraphNodeSelection::None;
	}
}

void SMassProcessingGraphView::MarkDependencies(const FMassDebuggerProcessingGraphNode& Node)
{
	for (const int32 DependencyIndex : Node.WaitForNodes)
	{
		AllNodes[DependencyIndex]->Node.GraphNodeSelection = EMassDebuggerProcessingGraphNodeSelection::WaitFor;
	}
	for (const int32 DependencyIndex : Node.BlockNodes)
	{
		AllNodes[DependencyIndex]->Node.GraphNodeSelection = EMassDebuggerProcessingGraphNodeSelection::Block;
	}
}

#undef LOCTEXT_NAMESPACE
