// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "SMassDebuggerViewBase.h"
#include "MassDebuggerModel.h"
#include "Widgets/SBoxPanel.h"

struct FMassDebuggerProcessingGraph;
struct FMassDebuggerModel;
template<typename T> class STreeView;

struct FMassDebuggerProcessingGraphNodeTreeItem
{
	FMassDebuggerProcessingGraphNodeTreeItem(const FMassDebuggerProcessingGraphNode& InNode);

	FMassDebuggerProcessingGraphNode Node;
	TArray<TSharedPtr<FMassDebuggerProcessingGraphNodeTreeItem>> ChildItems;
};

class SMassProcessingGraphView : public SMassDebuggerViewBase
{
public:
	SLATE_BEGIN_ARGS(SMassProcessingGraphView)
		: _OffsetPerLevel(10.f)
		{}
		SLATE_ATTRIBUTE(float, OffsetPerLevel)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<FMassDebuggerModel> InDebuggerModel);

	void Display(TSharedPtr<FMassDebuggerProcessingGraph> InProcessingGraphData);

protected:
	void HandleSelectionChanged(TSharedPtr<FMassDebuggerProcessingGraphNodeTreeItem> InNode, ESelectInfo::Type InSelectInfo);

	virtual void OnRefresh() override;
	virtual void OnProcessorsSelected(TConstArrayView<TSharedPtr<FMassDebuggerProcessorData>> SelectedProcessors, ESelectInfo::Type) override;
	virtual void OnArchetypesSelected(TConstArrayView<TSharedPtr<FMassDebuggerArchetypeData>> SelectedArchetypes, ESelectInfo::Type) override;

	void ClearSelection();
	void MarkDependencies(const FMassDebuggerProcessingGraphNode& Node);

	TSharedPtr<FMassDebuggerProcessingGraph> ProcessingGraphData;
	TSharedPtr<SVerticalBox> ItemsBox;
	float OffsetPerLevel;

	TArray<TSharedPtr<FMassDebuggerProcessingGraphNodeTreeItem>> AllNodes;
	TArray<TSharedPtr<FMassDebuggerProcessingGraphNodeTreeItem>> RootNodes;

	TSharedPtr<STreeView<TSharedPtr<FMassDebuggerProcessingGraphNodeTreeItem>>> GraphNodesTree;
};
