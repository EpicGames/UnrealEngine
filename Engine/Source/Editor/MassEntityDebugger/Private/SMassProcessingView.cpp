// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMassProcessingView.h"
#include "MassDebuggerModel.h"
#include "SMassProcessingGraphView.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SMassDebugger"

//----------------------------------------------------------------------//
// SMassProcessingGraphListTableRow
//----------------------------------------------------------------------//
using FMassDebuggerProcessingGraphPtr = TSharedPtr<FMassDebuggerProcessingGraph, ESPMode::ThreadSafe>;

class SMassProcessingGraphListTableRow : public STableRow<FMassDebuggerProcessingGraphPtr>
{
public:
	SLATE_BEGIN_ARGS(SMassProcessingGraphListTableRow) { }
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const FMassDebuggerProcessingGraphPtr InEntryItem)
	{
		Item = InEntryItem;

		STableRow<FMassDebuggerProcessingGraphPtr>::Construct(STableRow<FMassDebuggerProcessingGraphPtr>::FArguments(), InOwnerTableView);

		ChildSlot
			[
				SNew(STextBlock)
				.Text(FText::FromString(Item->Label))
			];
	}

	FMassDebuggerProcessingGraphPtr Item;
};

//----------------------------------------------------------------------//
// SMassProcessingView
//----------------------------------------------------------------------//
SMassProcessingView::~SMassProcessingView()
{
	DebuggerModel->OnRefreshDelegate.Remove(OnRefreshHandle);
}

void SMassProcessingView::Construct(const FArguments& InArgs, TSharedRef<FMassDebuggerModel> InDebuggerModel)
{
	DebuggerModel = InDebuggerModel;

	ChildSlot
	[
		SNew(SSplitter)
		.Orientation(Orient_Horizontal)
	 
		+ SSplitter::Slot()
		.Value(.3f)
		.MinSize(250.0f)
		[
			SAssignNew(GraphsListWidget, SListView<TSharedPtr<FMassDebuggerProcessingGraph>>)
			.ListItemsSource(&DebuggerModel->CachedProcessingGraphs)
			.SelectionMode(ESelectionMode::Single)
			.OnGenerateRow_Lambda([](TSharedPtr<FMassDebuggerProcessingGraph> Item, const TSharedPtr<STableViewBase>& OwnerTable)
				{
					return SNew(SMassProcessingGraphListTableRow, OwnerTable.ToSharedRef(), Item);
				})
			.OnSelectionChanged_Lambda([this](TSharedPtr<FMassDebuggerProcessingGraph> SelectedItem, ESelectInfo::Type SelectInfo)
				{
					if (SelectedItem)
					{
						SelectedLabel = SelectedItem->Label;
					}
					else
					{
						SelectedLabel.Empty();
					}
					ProcessingGraphWidget->Display(SelectedItem);
				})
		]
		
		+ SSplitter::Slot()
		.Value(.7f)
		[
			SAssignNew(ProcessingGraphWidget, SMassProcessingGraphView, InDebuggerModel)
		]
	];

	OnRefreshHandle = DebuggerModel->OnRefreshDelegate.AddRaw(this, &SMassProcessingView::OnRefresh);
}

void SMassProcessingView::OnRefresh()
{
	GraphsListWidget->RequestListRefresh();
	if (DebuggerModel.IsValid() && DebuggerModel->CachedProcessingGraphs.Num() > 0)
	{
		int SelectedIndex = 0;

		if (!SelectedLabel.IsEmpty())
		{
			for (int i = 0; i < DebuggerModel->CachedProcessingGraphs.Num(); i++)
			{
				if (DebuggerModel->CachedProcessingGraphs[i]->Label == SelectedLabel)
				{
					SelectedIndex = i;
					break;
				}
			}
		}

		GraphsListWidget->SetSelection(DebuggerModel->CachedProcessingGraphs[SelectedIndex]);
	}
}

#undef LOCTEXT_NAMESPACE
