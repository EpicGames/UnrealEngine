// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/SCompoundWidget.h"


struct FMassDebuggerModel;
struct FMassDebuggerProcessingGraph;
class SMassProcessingGraphView;
template<typename T> class SListView;

class SMassProcessingView : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMassProcessingView){}
	SLATE_END_ARGS()

	~SMassProcessingView();

	void Construct(const FArguments& InArgs, TSharedRef<FMassDebuggerModel> InDebuggerModel);

protected:
	void OnRefresh();

	TSharedPtr<FMassDebuggerModel> DebuggerModel;

	TSharedPtr<SListView<TSharedPtr<FMassDebuggerProcessingGraph>>> GraphsListWidget;

	TSharedPtr<SMassProcessingGraphView> ProcessingGraphWidget;
	FString SelectedLabel;

	FDelegateHandle OnRefreshHandle;
};
