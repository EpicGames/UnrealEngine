// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMassProcessor.h"
#include "MassDebuggerModel.h"
#include "MassDebuggerStyle.h"
#include "SMassQuery.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "SMassDebugger"

//----------------------------------------------------------------------//
// SMassProcessor
//----------------------------------------------------------------------//
void SMassProcessor::Construct(const FArguments& InArgs, TSharedPtr<FMassDebuggerProcessorData> InProcessorData, TSharedRef<FMassDebuggerModel> InDebuggerModel)
{
	ProcessorData = InProcessorData;
	if (!ProcessorData)
	{
		return;
	}

	TSharedRef<SVerticalBox> Box = SNew(SVerticalBox);

#if WITH_MASSENTITY_DEBUG
	if (!ProcessorData->Description.IsEmpty())
	{
		Box->AddSlot()
		.AutoHeight()
		[
			SNew(SBorder)
			.Padding(10.0f)
			[
				SNew(STextBlock)
				.Text(FText::FromString(ProcessorData->Description))
			]
		];
	}
	
	if (ProcessorData->ProcessorRequirements->IsEmpty())
	{
		Box->AddSlot()
		.AutoHeight()
		[
			SNew(SBorder)
			.Padding(10.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("NoProcessorRequirements", "No Processor Requirements"))
			]
		];
	}
	else
	{
		Box->AddSlot()
		.AutoHeight()
		[
			SNew(SMassQuery, ProcessorData->ProcessorRequirements, InDebuggerModel)
			.EditName(FString::Format(TEXT("{0}:{1}"), { ProcessorData->Label, ProcessorData->ProcessorRequirements->Label.ToString() }))
		];
	}

	for (TSharedPtr<FMassDebuggerQueryData>& QueryData : ProcessorData->Queries)
	{
		Box->AddSlot()
		.AutoHeight()
		[
			SNew(SMassQuery, QueryData, InDebuggerModel)
			.EditName(FString::Format(TEXT("{0}:{1}"), { ProcessorData->Label, QueryData->Label.ToString()}))
		];
	}

#endif //WITH_MASSENTITY_DEBUG

	ChildSlot
	[
		Box
	];
}

#undef LOCTEXT_NAMESPACE

