// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/SCompoundWidget.h"

struct FMassDebuggerModel;
struct FMassDebuggerProcessorData;

class SMassProcessor : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMassProcessor){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FMassDebuggerProcessorData> InProcessorData, TSharedRef<FMassDebuggerModel> InDebuggerModel);

protected:
	TSharedPtr<FMassDebuggerProcessorData> ProcessorData;
};
