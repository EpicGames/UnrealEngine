// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMassProcessorsView.h"
#include "SMassProcessor.h"
#include "SMassProcessorsWidget.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SMassDebugger"

namespace UE::Mass::Debug::UI::Private
{
	const FText PickProcessorLabel = FText::FromString(TEXT("Pick a processor from the list"));
	const FText MissingDebugData = FText::FromString(TEXT("Missing debug data"));
}


//----------------------------------------------------------------------//
// SMassProcessorListTableRow
//----------------------------------------------------------------------//
class SMassProcessorListTableRow : public STableRow<TSharedPtr<FMassDebuggerProcessorData>>
{
public:
	using Super = STableRow<TSharedPtr<FMassDebuggerProcessorData>>;

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedPtr<FMassDebuggerProcessorData>& InEntryItem
		, TSharedRef<FMassDebuggerModel> DebuggerModel)
	{
		if (!InEntryItem.IsValid())
		{
			return;
		}

		Item = InEntryItem;
		
		Super::Construct(Super::FArguments(), InOwnerTableView);
		
		ChildSlot
		[
			SNew(SBox)
			.Padding(2.0f)
			[
				SNew(SMassProcessorWidget, Item.ToSharedRef(), DebuggerModel)
			]
		];
	}

	TSharedPtr<FMassDebuggerProcessorData> Item;
};

//----------------------------------------------------------------------//
// SMassProcessorCollectionListView
//----------------------------------------------------------------------//
 /**
  * we have multiple instances of SMassProcessorCollectionListView at one time, and we want selection
  * cleared on all of them when any of them gets cleared. This class lets us do it by overriding
  * Private_ClearSelection and notifying the main view about the fact.
  */
class SMassProcessorCollectionListView : public SListView<TSharedPtr<FMassDebuggerProcessorData>>
{
public:
	using Super = SListView<TSharedPtr<FMassDebuggerProcessorData>>;

	void Construct(const FArguments& InArgs, const TWeakPtr<SMassProcessorsView>& InWeakMainView)
	{
		WeakMainView = InWeakMainView;
		Super::Construct(InArgs);
	}

	virtual void Private_ClearSelection() override
	{
		if (TSharedPtr<SMassProcessorsView> SharedMainView = WeakMainView.Pin())
		{
			SharedMainView->OnClearSelection(*this);
		}
		SListView::Private_ClearSelection();
	}

protected:
	TWeakPtr<SMassProcessorsView> WeakMainView;
};

//----------------------------------------------------------------------//
// SMassProcessorCollectionTableRow
//----------------------------------------------------------------------//
class SMassProcessorCollectionTableRow : public STableRow<TSharedPtr<FMassDebuggerModel::FProcessorCollection>>
{
public:
	using Super = STableRow<TSharedPtr<FMassDebuggerModel::FProcessorCollection>>;
	using FItemType = TSharedPtr<FMassDebuggerProcessorData>;

	void Construct(const FArguments&, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedPtr<FMassDebuggerModel::FProcessorCollection>& InEntryItem
		, const TSharedRef<SMassProcessorsView>& MainView, TSharedRef<FMassDebuggerModel> DebuggerModel)
	{
		CollectionItem = InEntryItem;

		Super::Construct(Super::FArguments(), InOwnerTableView);

		TSharedPtr<SMassProcessorCollectionListView>& ProcessorsListWidget = MainView->ProcessorsListWidgets.AddDefaulted_GetRef();

		ChildSlot
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Top)
			[
				SNew(STextBlock)
				.Text(FText::FromName(CollectionItem->Label))
				.TextStyle(FAppStyle::Get(), "LargeText")
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Top)
			[
				SAssignNew(ProcessorsListWidget, SMassProcessorCollectionListView, MainView)
				.ListItemsSource(&CollectionItem->Container)
				.SelectionMode(ESelectionMode::None)
				.OnSelectionChanged(MainView->OnProcessorSelectionChanged)
				.OnGenerateRow_Lambda([DebuggerModel](FItemType Item, const TSharedRef<STableViewBase>& OwnerTable)
				{
					return SNew(SMassProcessorListTableRow, OwnerTable, Item, DebuggerModel);
				})
			]
		];
	}

protected:
	TSharedPtr<FMassDebuggerModel::FProcessorCollection> CollectionItem;
};

//----------------------------------------------------------------------//
// SMassProcessorsView
//----------------------------------------------------------------------//
void SMassProcessorsView::Construct(const FArguments& InArgs, const TSharedRef<FMassDebuggerModel>& InDebuggerModel)
{
	Initialize(InDebuggerModel);


	ChildSlot
	[
		SNew(SSplitter)
		.Orientation(Orient_Horizontal)
			 
		+ SSplitter::Slot()
		.Value(.35f)
		.MinSize(260.0f)
		[
			SNew(SScrollBox)
			.Orientation(Orient_Vertical)
			+ SScrollBox::Slot()
			.VAlign(VAlign_Top)
			[
				SAssignNew(ProcessorCollectionsListWidget, SListView<TSharedPtr<FMassDebuggerModel::FProcessorCollection>>)
				.ListItemsSource(&DebuggerModel->CachedProcessorCollections)
				.SelectionMode(ESelectionMode::None)
				.Orientation(Orient_Horizontal)
				.OnGenerateRow_Lambda([SharedThis = StaticCastSharedRef<SMassProcessorsView>(AsShared()), InDebuggerModel]
					(TSharedPtr<FMassDebuggerModel::FProcessorCollection> Item, const TSharedRef<STableViewBase>& OwnerTable)
					{
						return SNew(SMassProcessorCollectionTableRow, OwnerTable, Item, SharedThis, InDebuggerModel);
					})
			]
		]
	];

	PopulateProcessorList();
}


void SMassProcessorsView::ProcessorListSelectionChanged(TSharedPtr<FMassDebuggerProcessorData> SelectedItem, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo == ESelectInfo::Direct)
	{
		return;
	}
	
	check(DebuggerModel);

	TArray<TSharedPtr<FMassDebuggerProcessorData>> CurrentlySelectedProcessors;
	for (const auto& Widget : ProcessorsListWidgets)
	{
		TArray<TSharedPtr<FMassDebuggerProcessorData>> LocalSelectedProcessors;
		Widget->GetSelectedItems(LocalSelectedProcessors);
		CurrentlySelectedProcessors.Append(MoveTemp(LocalSelectedProcessors));
	}
	DebuggerModel->SelectProcessors(CurrentlySelectedProcessors, SelectInfo);
}

void SMassProcessorsView::OnClearSelection(const SMassProcessorCollectionListView& TransientSource)
{
	if (bClearingSelection)
	{
		return;
	}

	TGuardValue<bool> GuardValue(bClearingSelection, true);
	
	for (const auto& Widget : ProcessorsListWidgets)
	{
		if (Widget.Get() != &TransientSource)
		{
			Widget->ClearSelection();
		}
	}
}

void SMassProcessorsView::OnProcessorsSelected(TConstArrayView<TSharedPtr<FMassDebuggerProcessorData>> SelectedProcessors, ESelectInfo::Type SelectInfo)
{
	using namespace UE::Mass::Debug::UI::Private;
	using namespace UE::Mass::Debug;
	
	if (!DebuggerModel)
	{
		return;
	}

	if (SelectInfo == ESelectInfo::Direct)
	{
		for (const auto& Widget : ProcessorsListWidgets)
		{
			Widget->ClearSelection();
		}
	}

	if (SelectedProcessors.Num())
	{		
		if (SelectInfo == ESelectInfo::Direct)
		{
			for (const auto& Widget : ProcessorsListWidgets)
			{
				Widget->SetItemSelection(SelectedProcessors, /*bSelected=*/true, ESelectInfo::OnMouseClick);
			}
		}
	}

	ProcessorCollectionsListWidget->RequestListRefresh();
}

void SMassProcessorsView::OnArchetypesSelected(TConstArrayView<TSharedPtr<FMassDebuggerArchetypeData>> SelectedArchetypes, ESelectInfo::Type SelectInfo)
{
	OnProcessorsSelected(DebuggerModel->SelectedProcessors, ESelectInfo::Direct);
}

void SMassProcessorsView::OnRefresh()
{
	PopulateProcessorList();
}

void SMassProcessorsView::PopulateProcessorList()
{
	check(DebuggerModel.IsValid());
	DebuggerModel->ClearProcessorSelection();

	ProcessorCollectionsListWidget->RequestListRefresh();
}

#undef LOCTEXT_NAMESPACE

