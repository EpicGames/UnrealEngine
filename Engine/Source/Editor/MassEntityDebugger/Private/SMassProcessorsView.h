// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "SMassDebuggerViewBase.h"
#include "MassDebuggerModel.h"
#include "Templates/SharedPointer.h"

struct FMassDebuggerProcessorData;
struct FMassDebuggerArchetypeData;
class SRichTextBlock;
class SMassProcessor;
class SMassProcessorCollectionListView;
class SMassProcessorCollectionTableRow;
template<typename T> class SListView;
class SVerticalBox;

class SMassProcessorsView : public SMassDebuggerViewBase
{
public:
	SLATE_BEGIN_ARGS(SMassProcessorsView)
	{
	}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FMassDebuggerModel>& InDebuggerModel);

	void PopulateProcessorList();

protected:
	virtual void OnRefresh() override;
	virtual void OnProcessorsSelected(TConstArrayView<TSharedPtr<FMassDebuggerProcessorData>> SelectedProcessors, ESelectInfo::Type SelectInfo) override;
	virtual void OnArchetypesSelected(TConstArrayView<TSharedPtr<FMassDebuggerArchetypeData>> SelectedArchetypes, ESelectInfo::Type SelectInfo) override;

	void ProcessorListSelectionChanged(TSharedPtr<FMassDebuggerProcessorData> SelectedItem, ESelectInfo::Type SelectInfo);

	friend SMassProcessorCollectionListView;
	friend SMassProcessorCollectionTableRow;
	void OnClearSelection(const SMassProcessorCollectionListView& TransientSource);

	TSharedPtr<SListView<TSharedPtr<FMassDebuggerModel::FProcessorCollection>>> ProcessorCollectionsListWidget;
	TArray<TSharedPtr<SMassProcessorCollectionListView>> ProcessorsListWidgets;
	TSharedPtr<SMassProcessor> ProcessorWidget;
	TSharedPtr<SVerticalBox> ProcessorsBox;

	DECLARE_DELEGATE_TwoParams(FOnProcessorSelectionChanged, TSharedPtr<FMassDebuggerProcessorData>, ESelectInfo::Type);
	FOnProcessorSelectionChanged OnProcessorSelectionChanged;

	bool bClearingSelection = false;
};
