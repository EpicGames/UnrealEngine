// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMassProcessorsWidget.h"
#include "SMassProcessor.h"
#include "Styling/AppStyle.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "SourceCodeNavigation.h"
#include "MassDebuggerStyle.h"

#define LOCTEXT_NAMESPACE "SMassDebugger"

SMassProcessorWidget::~SMassProcessorWidget()
{
	if (DebuggerModel.IsValid())
	{
		DebuggerModel->OnFragmentSelectedDelegate.Remove(OnFragmentSelectChangeHandle);
	}
}

void SMassProcessorWidget::Construct(const FArguments& InArgs, TSharedPtr<FMassDebuggerProcessorData> InDebuggerProcessorData, TSharedRef<FMassDebuggerModel> InDebuggerModel)
{
	ProcessorData = InDebuggerProcessorData;
	DebuggerModel = InDebuggerModel;

	OnFragmentSelectChangeHandle = DebuggerModel->OnFragmentSelectedDelegate.AddSP(this, &SMassProcessorWidget::HandleFragmentSelected);

	bIsExpandedText = false;
	bIsExpandedGraph = false;

	if (!ProcessorData.IsValid())
	{
		ChildSlot
		[
			SNew(STextBlock)
			.TextStyle(FAppStyle::Get(), "LargeText")
			.Text(LOCTEXT("InvalidDebugProcessorData", "Invalid Debug Processor Data"))
		];
		return;
	}

	FMassDebuggerProcessorData& ProcData = *ProcessorData.Get();

	ChildSlot
	[
		SAssignNew(Border, SBorder)
		.BorderImage(GetBorderByFragmentSelection())
		.BorderBackgroundColor(FLinearColor::Gray)
		.Padding(1.0f)
		[
			SNew(SBorder)
			.BorderImage(FMassDebuggerStyle::GetBrush("MassDebug.Processor.InnerBackground"))
			.Padding(2.0f)
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "NoBorder")
						.Content()
						[
							SNew(STextBlock)
							.TextStyle(FAppStyle::Get(), "NormalText")
							.Text(FText::FromString(ProcData.Label))
						]
						.OnClicked(FOnClicked::CreateSP(this, &SMassProcessorWidget::HandleSelectProcessorClicked))
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "NoBorder")
						.ToolTipText(LOCTEXT("ShowFragmentAccess", "Show Fragment Access"))
						.Content()
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.Layout"))
						]
						.OnClicked(FOnClicked::CreateSP(this, &SMassProcessorWidget::HandleExpandTextClicked))
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "NoBorder")
						.ToolTipText(LOCTEXT("ShowStats", "Show Stats"))
						.Content()
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.LOD"))
						]
						.OnClicked(FOnClicked::CreateSP(this, &SMassProcessorWidget::HandleExpandGraphClicked))
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "NoBorder")
						.ToolTipText(LOCTEXT("OpenSourceLocation", "Open Source Location"))
						.Content()
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.C++"))
						]
						.OnClicked(FOnClicked::CreateSP(this, &SMassProcessorWidget::HandleOpenSourceLocationClicked))
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "NoBorder")
						.ToolTipText(LOCTEXT("ShowAffectedEntities", "Show Affected Entities"))
						.Content()
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.Search"))
						]
						.OnClicked(FOnClicked::CreateSP(this, &SMassProcessorWidget::HandleShowEntitiesClicked))
					]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SAssignNew(TextBoxContainer, SBox)
					.Visibility(bIsExpandedText ? EVisibility::Visible : EVisibility::Collapsed)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("DetailedProcessorInformation", "Detailed information about the processor..."))
					]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SAssignNew(GraphBoxContainer, SBox)
					.Visibility(bIsExpandedGraph ? EVisibility::Visible : EVisibility::Collapsed)
					[
						// This is a placeholder for the stats/perf graph widget
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.LOD"))
					]
				]
			]
		]
	];
}

FReply SMassProcessorWidget::HandleSelectProcessorClicked()
{
	return FReply::Handled();
}

FReply SMassProcessorWidget::HandleExpandTextClicked()
{
	bIsExpandedText = !bIsExpandedText;
	if (TextBoxContainer.IsValid() && DebuggerModel.IsValid())
	{
		TextBoxContainer->SetContent(SNew(SMassProcessor, ProcessorData, DebuggerModel.ToSharedRef()));
		TextBoxContainer->SetVisibility(bIsExpandedText ? EVisibility::Visible : EVisibility::Collapsed);
	}
	return FReply::Handled();
}

FReply SMassProcessorWidget::HandleExpandGraphClicked()
{
	bIsExpandedGraph = !bIsExpandedGraph;
	if (GraphBoxContainer.IsValid())
	{
		GraphBoxContainer->SetVisibility(bIsExpandedGraph ? EVisibility::Visible : EVisibility::Collapsed);
	}
	return FReply::Handled();
}

FReply SMassProcessorWidget::HandleOpenSourceLocationClicked()
{
	if (!ProcessorData.IsValid())
	{
		return FReply::Handled();
	}

	const UMassProcessor* Proc = ProcessorData.Get()->Processor.Get();
	if (Proc)
	{
		FSourceCodeNavigation::NavigateToClass(Proc->GetClass());
	}
	
	return FReply::Handled();
}

FReply SMassProcessorWidget::HandleShowEntitiesClicked()
{
#if WITH_MASSENTITY_DEBUG
	if (!ProcessorData.IsValid() || !DebuggerModel.IsValid())
	{
		return FReply::Handled();
	}

	const UMassProcessor* Proc = ProcessorData.Get()->Processor.Get();
	if (Proc)
	{
		TConstArrayView<FMassEntityQuery*> Queries = FMassDebugger::GetProcessorQueries(*Proc);
		DebuggerModel->ShowEntitiesView(0, Queries);
	}
#endif
	return FReply::Handled();
}

void SMassProcessorWidget::HandleFragmentSelected(FName SelectedFragment)
{
	Border->SetBorderImage(GetBorderByFragmentSelection());
}

const FSlateBrush* SMassProcessorWidget::GetBorderByFragmentSelection()
{
	enum AccessLevel
	{
		None = 0,
		Require,
		Read,
		Write,
		Block,
		MAX
	};

	const FSlateBrush* AccessColor[] = {
		FMassDebuggerStyle::GetBrush("MassDebug.Processor"),
		FMassDebuggerStyle::GetBrush("MassDebug.Processor.AccessRequired"),
		FMassDebuggerStyle::GetBrush("MassDebug.Processor.AccessRead"),
		FMassDebuggerStyle::GetBrush("MassDebug.Processor.AccessWrite"),
		FMassDebuggerStyle::GetBrush("MassDebug.Processor.AccessBlock")
	};

	static_assert(UE_ARRAY_COUNT(AccessColor) == int32(AccessLevel::MAX), "Ensure AccessLevel enum is 1:1 with the brush array");

	FName SelectedFragment;
	
	if (DebuggerModel.IsValid())
	{
		SelectedFragment = DebuggerModel->GetSelectedFragment();
	}

	auto HasSelectedBit = [&SelectedFragment](const auto& Bitset) -> bool {
#if WITH_MASSENTITY_DEBUG
		for (int32 Index = 0; Index < Bitset.StructTypesBitArray.Num(); ++Index)
		{
			if (Bitset.StructTypesBitArray[Index])
			{
				FName setName = Bitset.GetImplementation().GetStructTracker().DebugGetStructTypeName(Index);
				if (setName == SelectedFragment)
				{
					return true;
				}
			}
		}
#endif
		return false;
		};

	auto GetHighestAccessLevel = [&SelectedFragment, &HasSelectedBit](const auto& ExecutionAccess, AccessLevel Highest) -> AccessLevel {
		if (Highest < AccessLevel::Write && HasSelectedBit(ExecutionAccess.Write))
		{
			return AccessLevel::Write;
		}
		if (Highest < AccessLevel::Read && HasSelectedBit(ExecutionAccess.Read))
		{
			return AccessLevel::Read;
		}
		return Highest;
		};

	AccessLevel HighestAccess = AccessLevel::None;

	if (!SelectedFragment.IsNone())
	{
		for (TSharedPtr<FMassDebuggerQueryData>& Query : ProcessorData->Queries)
		{
			FMassExecutionRequirements& ExecutionReqs = Query->ExecutionRequirements;
			if (HighestAccess < AccessLevel::Block && HasSelectedBit(ExecutionReqs.RequiredNoneTags))
			{
				HighestAccess = AccessLevel::Block;
			}
			if (HighestAccess < AccessLevel::Require && (HasSelectedBit(ExecutionReqs.RequiredAnyTags) || HasSelectedBit(ExecutionReqs.RequiredAllTags)))
			{
				HighestAccess = AccessLevel::Require;
			}
			else
			{
				HighestAccess = GetHighestAccessLevel(ExecutionReqs.Fragments, HighestAccess);
				HighestAccess = GetHighestAccessLevel(ExecutionReqs.ChunkFragments, HighestAccess);
				HighestAccess = GetHighestAccessLevel(ExecutionReqs.SharedFragments, HighestAccess);
				HighestAccess = GetHighestAccessLevel(ExecutionReqs.RequiredSubsystems, HighestAccess);

				if (HighestAccess < AccessLevel::Read && HasSelectedBit(ExecutionReqs.ConstSharedFragments.Read))
				{
					HighestAccess = AccessLevel::Read;
				}
			}
		}
	}

	return AccessColor[HighestAccess];
}

#undef LOCTEXT_NAMESPACE