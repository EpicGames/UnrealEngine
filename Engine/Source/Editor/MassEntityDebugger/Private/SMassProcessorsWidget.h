// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "SMassProcessorsView.h"
#include "SMassProcessor.h"

class SBox;
class SBorder;

struct FMassDebuggerProcessorData;
struct FMassDebuggerModel;

class SMassProcessorWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMassProcessorWidget) {}
	SLATE_END_ARGS()
	virtual ~SMassProcessorWidget();

	void Construct(const FArguments& InArgs, TSharedPtr<FMassDebuggerProcessorData> InDebuggerProcessorData, TSharedRef<FMassDebuggerModel> InDebuggerModel);

private:
	FReply HandleExpandTextClicked();
	FReply HandleExpandGraphClicked();
	FReply HandleSelectProcessorClicked();
	FReply HandleOpenSourceLocationClicked();
	FReply HandleShowEntitiesClicked();
	void HandleFragmentSelected(FName SelectedFragment);
	bool bIsExpandedText;
	bool bIsExpandedGraph;
	TSharedPtr<SBox> TextBoxContainer;
	TSharedPtr<SBox> GraphBoxContainer;
	TSharedPtr<SBorder> Border;
	TSharedPtr<FMassDebuggerProcessorData> ProcessorData;
	TSharedPtr<FMassDebuggerModel> DebuggerModel;
	const FSlateBrush* GetBorderByFragmentSelection();
	FDelegateHandle OnFragmentSelectChangeHandle;
};
