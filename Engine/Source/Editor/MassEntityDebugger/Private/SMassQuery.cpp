// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMassQuery.h"
#include "SMassBitSet.h"
#include "MassEntityTypes.h"
#include "MassDebuggerModel.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "SMassDebugger"


static_assert(EMassAccessOperation::Read == uint32(EMassBitSetDisplayMode::ReadOnly) && EMassAccessOperation::Write == uint32(EMassBitSetDisplayMode::ReadWrite)
	, "EMassAccessOperation and EMassBitSetDisplayMode need to be kept in sync");

namespace UE::Mass::Debugger::UI::Private
{
	template<typename TBitSet>
	void AddBitSetPair(TSharedRef<SVerticalBox>& Box, const TMassExecutionAccess<TBitSet>& BitSetAccess, const FString& Label, TSharedRef<FMassDebuggerModel> InDebuggerModel)
	{
		if (BitSetAccess.IsEmpty() == false)
		{
			static const FSlateBrush* RequiredAccessBrushes[] = {
				FMassDebuggerStyle::GetBrush("MassDebug.Fragment.ReadOnly")
				, FMassDebuggerStyle::GetBrush("MassDebug.Fragment.ReadWrite")
			};

			Box->AddSlot()
			.AutoHeight()
			[
				SNew(SMassBitSet<TBitSet>, InDebuggerModel, Label, BitSetAccess.AsArrayView(), RequiredAccessBrushes)
				.SlotPadding(5.0f)
			];
		}
	}
} // namespace UE::Mass::Debugger::UI::Private


void SMassQuery::Construct(const FArguments& InArgs, TSharedPtr<FMassDebuggerQueryData> InQueryData, TSharedRef<FMassDebuggerModel> InDebuggerModel)
{
	using UE::Mass::Debugger::UI::Private::AddBitSetPair;
	using UE::Mass::Debugger::UI::AddBitSet;

	const FSlateColor ReadOnlyColor = FLinearColor(0.4f, 0.4f, 0.4f, 0.9f);
	const FSlateColor ReadWriteColor = FLinearColor(0.1f, 0.8f, 0.1f, 0.9f);

	QueryData = InQueryData;

	FMassExecutionRequirements& ExecutionRequirements = InQueryData->ExecutionRequirements;
	TWeakPtr<FMassDebuggerModel> WeakDebuggerModel = InDebuggerModel;
	FString EditName = InArgs._EditName.IsSet() ? InArgs._EditName.Get() : InQueryData->Label.ToString();

	TSharedRef<SVerticalBox> Box = SNew(SVerticalBox);
	static const FSlateBrush* ReadOnlyBrush = FMassDebuggerStyle::GetBrush("MassDebug.Fragment.ReadOnly");

	Box->AddSlot()
	.AutoHeight()
	[
        SNew(SHorizontalBox)

        + SHorizontalBox::Slot()
        .AutoWidth()
        .VAlign(VAlign_Center)
        [
            SNew(SRichTextBlock)
            .Text(InQueryData->Label)
            .DecoratorStyleSet(&FAppStyle::Get())
            .TextStyle(FAppStyle::Get(), "LargeText")
        ]
        + SHorizontalBox::Slot()
        .FillWidth(1.0f)
        [
            SNullWidget::NullWidget
        ]
        + SHorizontalBox::Slot()
        .AutoWidth()
        .VAlign(VAlign_Center)
        [
            SNew(SButton)
            .Text(LOCTEXT("OpenInQueryEditor", "Open In Query Editor"))
            .OnClicked_Lambda([WeakDebuggerModel, QueryDataPtr = InQueryData, EditName]()
            {
				if(TSharedPtr<FMassDebuggerModel> SharedModel = WeakDebuggerModel.Pin())
				{
					SharedModel->ShowQueryInEditor(QueryDataPtr->SourceQuery, EditName);
				}
                return FReply::Handled();
            })
        ]
	];

	AddBitSetPair(Box, ExecutionRequirements.Fragments, TEXT("Fragments"), InDebuggerModel);
	AddBitSet(Box, ExecutionRequirements.RequiredAllTags, TEXT("RequiredAllTags"), ReadOnlyBrush, InDebuggerModel);
	AddBitSet(Box, ExecutionRequirements.RequiredAnyTags, TEXT("RequiredAnyTags"), ReadOnlyBrush, InDebuggerModel);
	AddBitSet(Box, ExecutionRequirements.RequiredNoneTags,TEXT("RequiredNoneTags"), ReadOnlyBrush, InDebuggerModel);
	AddBitSetPair(Box, ExecutionRequirements.ChunkFragments, TEXT("Chunk Fragments"), InDebuggerModel);
	AddBitSetPair(Box, ExecutionRequirements.SharedFragments, TEXT("Shared Fragments"), InDebuggerModel);
	AddBitSetPair(Box, ExecutionRequirements.ConstSharedFragments, TEXT("Const Shared Fragments"), InDebuggerModel);
	AddBitSetPair(Box, ExecutionRequirements.RequiredSubsystems, TEXT("Required Subsystems"), InDebuggerModel);

	if (InQueryData->AdditionalInformation.IsEmpty() == false)
	{
		Box->AddSlot()
		.AutoHeight()
		[
			SNew(SRichTextBlock)
			.Text(InQueryData->AdditionalInformation)
		];
	}

	ChildSlot
	[
		SNew(SBorder)
		.Padding(10.0f)
		[
			Box
		]
	];
}

#undef LOCTEXT_NAMESPACE

