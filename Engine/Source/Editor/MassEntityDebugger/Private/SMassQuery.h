// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/SCompoundWidget.h"

struct FMassDebuggerModel;
struct FMassDebuggerQueryData;

class SMassQuery : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMassQuery){}
		SLATE_ATTRIBUTE(FString, EditName)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FMassDebuggerQueryData> InQueryData, TSharedRef<FMassDebuggerModel> InDebuggerModel);

protected:
	TSharedPtr<FMassDebuggerQueryData> QueryData;
};
