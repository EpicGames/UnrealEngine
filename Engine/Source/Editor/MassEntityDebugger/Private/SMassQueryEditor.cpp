// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMassQueryEditor.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "SMassQueryEditorView.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Internationalization/Internationalization.h"

#define LOCTEXT_NAMESPACE "SMassRequirementsEditor"

namespace UE::MassDebugger
{

	void SQueryEditor::Construct(const FArguments& InArgs)
	{
		DebuggerModel = InArgs._DebuggerModel;
		OnRequirementsChanged = InArgs._OnRequirementsChanged;

		TWeakPtr<SQueryEditor> WeakThisPtr = StaticCastWeakPtr<SQueryEditor>(AsWeak());

		if (TSharedPtr<FMassDebuggerModel> Model = DebuggerModel.Pin())
		{
			Model->CacheFragmentData();
			Model->CacheTagData();

			for (const TSharedPtr<FMassDebuggerFragmentData>& FragData : Model->CachedFragmentData)
			{
				if (FragData->Fragment.IsValid())
				{
					int32 Index = AvailableFragmentNames.Add(MakeShared<FString>(FragData->Fragment->GetName()));
					FragmentNamesToTypes.Add(*AvailableFragmentNames[Index], FragData->Fragment);
				}
			}
			for (const TSharedPtr<FMassDebuggerFragmentData>& TagData : Model->CachedTagData)
			{
				if (TagData->Fragment.IsValid())
				{
					int32 Index = AvailableTagNames.Add(MakeShared<FString>(TagData->Fragment->GetName()));
					TagNamesToTypes.Add(*AvailableTagNames[Index], TagData->Fragment);
				}
			}
		}

		const UEnum* FragmentAccessEnum = StaticEnum<EMassFragmentAccess>();
		for (int32 i = 0; i < FragmentAccessEnum->NumEnums() - 1; ++i)
		{
			int64 AccessValue = FragmentAccessEnum->GetValueByIndex(i);
			if (AccessValue != INDEX_NONE)
			{
				FragmentAccessNames.Add(MakeShared<FName>(FragmentAccessEnum->GetNameByIndex(i)));
				FragmentAccessMap.Add(FragmentAccessEnum->GetNameByIndex(i), static_cast<EMassFragmentAccess>(AccessValue));
			}
		}

		const UEnum* FragmentPresenceEnum = StaticEnum<EMassFragmentPresence>();
		for (int32 i = 0; i < FragmentPresenceEnum->NumEnums() - 1; ++i)
		{
			int64 PresenceValue = FragmentPresenceEnum->GetValueByIndex(i);
			if (PresenceValue != INDEX_NONE)
			{
				FragmentPresenceNames.Add(MakeShared<FName>(FragmentPresenceEnum->GetNameByIndex(i)));
				FragmentPresenceMap.Add(FragmentPresenceEnum->GetNameByIndex(i), static_cast<EMassFragmentPresence>(PresenceValue));
			}
		}

		Load();

		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f)
			[
				SNew(SEditableTextBox)
				.Text_Lambda([WeakThisPtr]()
				{
					if (TSharedPtr<SQueryEditor> SharedThis = WeakThisPtr.Pin())
					{
						return SharedThis->Query.IsValid()
							? FText::FromString(SharedThis->Query->Name)
							: FText::GetEmpty();
					}

					return FText::GetEmpty();
				})
				.OnTextCommitted(this, &SQueryEditor::HandleNameTextCommitted)
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("FragmentsLabel", "Fragment Requirements"))
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SNew(SButton)
					.Text(LOCTEXT("AddFragBtn", "Add Fragment"))
					.OnClicked(this, &SQueryEditor::OnAddFragment)
				]
			]
			+ SVerticalBox::Slot()
			.FillHeight(0.5f)
			.Padding(2.0f)
			[
				SAssignNew(FragmentListView, SListView<TSharedPtr<FFragmentEntry>>)
				.ListItemsSource(&FragmentEntries)
				.OnGenerateRow(this, &SQueryEditor::OnGenerateFragmentRow)
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("TagsLabel", "Tag Requirements"))
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SNew(SButton)
					.Text(LOCTEXT("AddTagBtn", "Add Tag"))
					.OnClicked(this, &SQueryEditor::OnAddTag)
				]
			]
			+ SVerticalBox::Slot()
			.FillHeight(0.5f)
			.Padding(2.0f)
			[
				SAssignNew(TagListView, SListView<TSharedPtr<FFragmentEntry>>)
				.ListItemsSource(&TagEntries)
				.OnGenerateRow(this, &SQueryEditor::OnGenerateTagRow)
			]
		];
	}

	void SQueryEditor::SetQuery(TSharedPtr<FEditableQuery>& InQuery)
	{
		Query = InQuery;
		Load();
	}

	void SQueryEditor::ApplyChanges()
	{
		if (Query.IsValid())
		{
			Query->FragmentRequirements = FragmentEntries;
			Query->TagRequirements = TagEntries;
		}
		else
		{
			Load();
		}
	}

	void SQueryEditor::Load()
	{
		if (Query.IsValid())
		{
			FragmentEntries = Query->FragmentRequirements;
			TagEntries = Query->TagRequirements;
		}
		else
		{
			FragmentEntries.Reset();
			TagEntries.Empty();
		}

		if (FragmentListView.IsValid())
		{
			FragmentListView->RequestListRefresh();
		}
		if (TagListView.IsValid())
		{
			TagListView->RequestListRefresh();
		}
	}

	void SQueryEditor::HandleNameTextCommitted(const FText& InText, ETextCommit::Type InCommitType)
	{
		if (Query.IsValid())
		{
			Query->Name = InText.ToString();
			NotifyRequirementsChanged();
			if (TSharedPtr<FMassDebuggerModel> Model = DebuggerModel.Pin())
			{
				Model->RefreshQueries();
			}
		}
	}

	TSharedRef<ITableRow> SQueryEditor::OnGenerateFragmentRow(TSharedPtr<FFragmentEntry> InEntry, const TSharedRef<STableViewBase>& OwnerTable)
	{
		TSharedPtr<SQueryEditor> SharedThis = StaticCastSharedPtr<SQueryEditor>(AsShared().ToSharedPtr());
		return SNew(SFragmentSelectorRow, OwnerTable, InEntry, SharedThis.ToSharedRef(), false);
	}

	void SQueryEditor::SFragmentSelectorRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, const TSharedPtr<FFragmentEntry>& InEntry, TSharedRef<SQueryEditor> InOwnerRequirementsEditor, bool bInShowTags)
	{
		OwnerRequirementsEditor = InOwnerRequirementsEditor;
		ConstructInternal(InArgs, InOwnerTable);

		TArray<TSharedPtr<FString>>* FragmentNamesPtr;
		TSharedPtr<SWidget> AccessModeSelector;
		TWeakPtr<SQueryEditor::SFragmentSelectorRow> WeakThisPtr = StaticCastWeakPtr<SQueryEditor::SFragmentSelectorRow>(AsWeak());

		bShowTags = bInShowTags;
		if (bShowTags)
		{
			FragmentNamesPtr = &InOwnerRequirementsEditor->AvailableTagNames;
			AccessModeSelector = SNullWidget::NullWidget.ToSharedPtr();
		}
		else
		{
			FragmentNamesPtr = &InOwnerRequirementsEditor->AvailableFragmentNames;

			AccessModeSelector = SNew(SComboBox<TSharedPtr<FName>>)
			.OptionsSource(&InOwnerRequirementsEditor->FragmentAccessNames)
			.OnGenerateWidget_Lambda([](TSharedPtr<FName> InName)
			{
				return SNew(STextBlock).Text(FText::FromName(*InName));
			})
			.OnSelectionChanged_Lambda([WeakThisPtr, InEntry](TSharedPtr<FName> NewName, ESelectInfo::Type)
			{
				TSharedPtr<SQueryEditor::SFragmentSelectorRow> SharedThis = WeakThisPtr.Pin();
				if (SharedThis.IsValid() && InEntry.IsValid() && NewName.IsValid() && SharedThis->OwnerRequirementsEditor.IsValid())
				{
					SQueryEditor& Editor = *SharedThis->OwnerRequirementsEditor.Pin();
					InEntry->AccessMode = Editor.FragmentAccessMap[*NewName];
					Editor.NotifyRequirementsChanged();
				}
			})
			[
				SNew(STextBlock).Text_Lambda([InEntry]()
				{
					switch (InEntry->AccessMode)
					{
					case EMassFragmentAccess::ReadOnly:
						return LOCTEXT("ReadOnly", "ReadOnly");
					case EMassFragmentAccess::ReadWrite:
						return LOCTEXT("ReadWrite", "ReadWrite");
					default:
						return LOCTEXT("NoneAccess", "<None>");
					}
				})
			];
		}

		ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot().AutoWidth().Padding(2)
			[
				SAssignNew(FragmentsComboBox, SSearchableComboBox)
				.OptionsSource(FragmentNamesPtr)
				.OnGenerateWidget_Lambda([](const TSharedPtr<FString>& InItem)
				{
					return SNew(STextBlock)
						.Text(InItem ? FText::FromString(*InItem) : FText::GetEmpty());
				})
				.OnSelectionChanged_Lambda([WeakThisPtr, InEntry](const TSharedPtr<FString> Value, ESelectInfo::Type)
				{
					TSharedPtr<SQueryEditor::SFragmentSelectorRow> SharedThis = WeakThisPtr.Pin();
					if (SharedThis.IsValid() && InEntry.IsValid() && Value.IsValid() && SharedThis->OwnerRequirementsEditor.IsValid())
					{
						SQueryEditor& Editor = *SharedThis->OwnerRequirementsEditor.Pin();
						if (SharedThis->bShowTags)
						{
							InEntry->StructType = Editor.TagNamesToTypes[*Value];
						}
						else
						{
							InEntry->StructType = Editor.FragmentNamesToTypes[*Value];
						}
						Editor.NotifyRequirementsChanged();
					}
				})
				.Content()
				[
					SNew(STextBlock)
					.Text_Lambda([InEntry]() -> FText
					{
						if (InEntry.IsValid() && InEntry->StructType.IsValid())
						{
							return InEntry->StructType->GetDisplayNameText();
						}
						return LOCTEXT("NoneFragment", "<None>");
					})
				]


			]
			+ SHorizontalBox::Slot().AutoWidth().Padding(2)
			[
				AccessModeSelector.ToSharedRef()
			]
			+ SHorizontalBox::Slot().AutoWidth().Padding(2)
			[
				SNew(SComboBox<TSharedPtr<FName>>)
				.OptionsSource(&InOwnerRequirementsEditor->FragmentPresenceNames)
				.OnSelectionChanged_Lambda([WeakThisPtr, InEntry](TSharedPtr<FName> NewName, ESelectInfo::Type)
				{
					TSharedPtr<SQueryEditor::SFragmentSelectorRow> SharedThis = WeakThisPtr.Pin();
					if (SharedThis.IsValid() && InEntry.IsValid() && NewName.IsValid() && SharedThis->OwnerRequirementsEditor.IsValid())
					{
						SQueryEditor& Editor = *SharedThis->OwnerRequirementsEditor.Pin();
						InEntry->Presence = Editor.FragmentPresenceMap[*NewName];
						Editor.NotifyRequirementsChanged();
					}
				})
				.OnGenerateWidget_Lambda([](TSharedPtr<FName> InName)
				{
					return SNew(STextBlock).Text(FText::FromName(*InName));
				})
				[
					SNew(STextBlock).Text_Lambda([InEntry]()
					{
						if (!InEntry.IsValid()) return LOCTEXT("NonePres", "<None>");
						switch (InEntry->Presence)
						{
						case EMassFragmentPresence::All:
							return LOCTEXT("All", "All");
						case EMassFragmentPresence::Any:
							return LOCTEXT("Any", "Any");
						case EMassFragmentPresence::Optional:
							return LOCTEXT("Optional", "Optional");
						default:
							return LOCTEXT("None", "None");
						}
					})
				]
			]
			+ SHorizontalBox::Slot().AutoWidth().Padding(2)
			[
				SNew(SButton)
				.Text(LOCTEXT("Remove", "Remove"))
				.OnClicked(
					&InOwnerRequirementsEditor.Get(),
					bShowTags ? &SQueryEditor::OnRemoveTag : &SQueryEditor::OnRemoveFragment,
					InEntry)
			]
		];
	}

	TSharedRef<ITableRow> SQueryEditor::OnGenerateTagRow(TSharedPtr<FFragmentEntry> InEntry, const TSharedRef<STableViewBase>& OwnerTable)
	{
		TSharedPtr<SQueryEditor> SharedThis = StaticCastSharedPtr<SQueryEditor>(AsShared().ToSharedPtr());
		return SNew(SFragmentSelectorRow, OwnerTable, InEntry, SharedThis.ToSharedRef(), true);
	}

	FReply SQueryEditor::OnAddFragment()
	{
		TSharedPtr<FFragmentEntry> NewFragmentEntry = MakeShared<FFragmentEntry>();
		if (AvailableFragmentNames.Num() > 0)
		{
			NewFragmentEntry->StructType = (FragmentNamesToTypes[*AvailableFragmentNames[0]]);
		}
		FragmentEntries.Add(NewFragmentEntry);
		FragmentListView->RequestListRefresh();
		NotifyRequirementsChanged();
		return FReply::Handled();
	}

	FReply SQueryEditor::OnRemoveFragment(TSharedPtr<FFragmentEntry> InEntry)
	{
		FragmentEntries.Remove(InEntry);
		FragmentListView->RequestListRefresh();
		NotifyRequirementsChanged();
		return FReply::Handled();
	}

	FReply SQueryEditor::OnAddTag()
	{
		TSharedPtr<FFragmentEntry> NewTagEntry = MakeShared<FFragmentEntry>();
		if (AvailableTagNames.Num() > 0)
		{
			NewTagEntry->StructType = (TagNamesToTypes[*AvailableTagNames[0]]);
		}
		TagEntries.Add(NewTagEntry);
		TagListView->RequestListRefresh();
		NotifyRequirementsChanged();
		return FReply::Handled();
	}

	FReply SQueryEditor::OnRemoveTag(TSharedPtr<FFragmentEntry> InEntry)
	{
		TagEntries.Remove(InEntry);
		TagListView->RequestListRefresh();
		NotifyRequirementsChanged();
		return FReply::Handled();
	}

	void SQueryEditor::SFragmentSelectorRow::OnFragmentTypeChanged(TSharedPtr<TWeakObjectPtr<const UScriptStruct>> NewType, ESelectInfo::Type, TSharedPtr<FFragmentEntry> Entry)
	{
		if (Entry.IsValid() && NewType.IsValid() && NewType->Get() && OwnerRequirementsEditor.IsValid())
		{
			SQueryEditor& Editor = *OwnerRequirementsEditor.Pin();
			Entry->StructType = NewType->Get();
			Editor.NotifyRequirementsChanged();
		}
	}

	void SQueryEditor::SFragmentSelectorRow::OnFragmentAccessChanged(EMassFragmentAccess NewAccess, ESelectInfo::Type, TSharedPtr<FFragmentEntry> Entry)
	{
		if (Entry.IsValid() && OwnerRequirementsEditor.IsValid())
		{
			SQueryEditor& Editor = *OwnerRequirementsEditor.Pin();
			Entry->AccessMode = NewAccess;
			Editor.NotifyRequirementsChanged();
		}
	}

	void SQueryEditor::SFragmentSelectorRow::OnFragmentPresenceChanged(EMassFragmentPresence  NewPresence, ESelectInfo::Type, TSharedPtr<FFragmentEntry> Entry)
	{
		if (Entry.IsValid() && OwnerRequirementsEditor.IsValid())
		{
			SQueryEditor& Editor = *OwnerRequirementsEditor.Pin();
			Entry->Presence = NewPresence;
			Editor.NotifyRequirementsChanged();
		}
	}

	void SQueryEditor::SFragmentSelectorRow::OnTagTypeChanged(TSharedPtr<TWeakObjectPtr<const UScriptStruct>> NewType, ESelectInfo::Type, TSharedPtr<FFragmentEntry> Entry)
	{
		if (Entry.IsValid() && NewType.IsValid() && NewType->Get() && OwnerRequirementsEditor.IsValid())
		{
			SQueryEditor& Editor = *OwnerRequirementsEditor.Pin();
			Entry->StructType = NewType->Get();
			Editor.NotifyRequirementsChanged();
		}
	}

	void SQueryEditor::SFragmentSelectorRow::OnTagPresenceChanged(EMassFragmentPresence NewPresence, ESelectInfo::Type, TSharedPtr<FFragmentEntry> Entry)
	{
		if (Entry.IsValid() && OwnerRequirementsEditor.IsValid())
		{
			SQueryEditor& Editor = *OwnerRequirementsEditor.Pin();
			Entry->Presence = NewPresence;
			Editor.NotifyRequirementsChanged();
		}
	}

	FText SQueryEditor::SFragmentSelectorRow::GetFragmentTypeText(TSharedPtr<FFragmentEntry> Entry) const
	{
		return Entry.IsValid() && Entry->StructType.IsValid()
			? FText::FromName(Entry->StructType->GetFName())
			: LOCTEXT("NoneFragment", "<None>");
	}

	FText SQueryEditor::SFragmentSelectorRow::GetTagTypeText(TSharedPtr<FFragmentEntry> Entry) const
	{
		return Entry.IsValid() && Entry->StructType.IsValid()
			? FText::FromName(Entry->StructType->GetFName())
			: LOCTEXT("NoneTag", "<None>");
	}

	FText SQueryEditor::SFragmentSelectorRow::GetTagPresenceText(TSharedPtr<FFragmentEntry> Entry) const
	{
		if (!Entry.IsValid()) return LOCTEXT("NonePres", "<None>");
		switch (Entry->Presence)
		{
		case EMassFragmentPresence::All:
			return LOCTEXT("All", "All");
		case EMassFragmentPresence::Any:
			return LOCTEXT("Any", "Any");
		case EMassFragmentPresence::Optional:
			return LOCTEXT("Optional", "Optional");
		default:
			return LOCTEXT("None", "None");
		}
	}

	void SQueryEditor::NotifyRequirementsChanged()
	{
		ApplyChanges();
		
		if (OnRequirementsChanged.IsBound())
		{
			OnRequirementsChanged.Execute(Query);
		}
	}

} // namespace UE::MassDebugger

#undef LOCTEXT_NAMESPACE
