// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "MassRequirements.h"
#include "MassDebuggerModel.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class SSearchableComboBox;

namespace UE::MassDebugger
{
	struct FEditableQuery;
	struct FFragmentEntry;

	DECLARE_DELEGATE_OneParam(FRequirementsChanged, TSharedPtr<FEditableQuery>&);

	class SQueryEditor : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(SQueryEditor)
			: _DebuggerModel()
			, _InitialRequirements()
			{}
			SLATE_ARGUMENT(TSharedPtr<FMassDebuggerModel>, DebuggerModel)
			SLATE_ARGUMENT(FMassFragmentRequirements, InitialRequirements)
			SLATE_EVENT(FRequirementsChanged, OnRequirementsChanged)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);
		void SetQuery(TSharedPtr<FEditableQuery>& InQuery);

	private:
		void ApplyChanges();
		void Load();
		void HandleNameTextCommitted(const FText& InText, ETextCommit::Type InCommitType);

		TSharedRef<ITableRow> OnGenerateFragmentRow(TSharedPtr<FFragmentEntry> InEntry, const TSharedRef<STableViewBase>& OwnerTable);

		TSharedRef<ITableRow> OnGenerateTagRow(TSharedPtr<FFragmentEntry> InEntry, const TSharedRef<STableViewBase>& OwnerTable);

		FReply OnAddFragment();
		FReply OnRemoveFragment(TSharedPtr<FFragmentEntry> InEntry);

		FReply OnAddTag();
		FReply OnRemoveTag(TSharedPtr<FFragmentEntry> InEntry);

		void NotifyRequirementsChanged();

		TWeakPtr<FMassDebuggerModel> DebuggerModel;

		FRequirementsChanged OnRequirementsChanged;
		TSharedPtr<FEditableQuery> Query;

		TArray<TSharedPtr<FFragmentEntry>> FragmentEntries;
		TArray<TSharedPtr<FFragmentEntry>> TagEntries;

		TArray<TSharedPtr<FString>> AvailableFragmentNames;
		TMap<FString, TWeakObjectPtr<const UScriptStruct>> FragmentNamesToTypes;
		TArray<TSharedPtr<FString>> AvailableTagNames;
		TMap<FString, TWeakObjectPtr<const UScriptStruct>> TagNamesToTypes;

		friend class SFragmentSelectorRow;
		class SFragmentSelectorRow : public STableRow<TSharedPtr<FFragmentEntry>>
		{
		public:

			void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, const TSharedPtr<FFragmentEntry>& InEntry, TSharedRef<SQueryEditor> InOwnerRequirementsEditor, bool bInShowTags);

			TSharedPtr<SSearchableComboBox> FragmentsComboBox;

			void OnFragmentTypeChanged(TSharedPtr<TWeakObjectPtr<const UScriptStruct>> NewType, ESelectInfo::Type, TSharedPtr<FFragmentEntry> Entry);
			void OnFragmentAccessChanged(EMassFragmentAccess NewAccess, ESelectInfo::Type, TSharedPtr<FFragmentEntry> Entry);
			void OnFragmentPresenceChanged(EMassFragmentPresence NewPresence, ESelectInfo::Type, TSharedPtr<FFragmentEntry> Entry);
			void OnTagTypeChanged(TSharedPtr<TWeakObjectPtr<const UScriptStruct>> NewType, ESelectInfo::Type, TSharedPtr<FFragmentEntry> Entry);
			void OnTagPresenceChanged(EMassFragmentPresence NewPresence, ESelectInfo::Type, TSharedPtr<FFragmentEntry> Entry);

			FText GetFragmentTypeText(TSharedPtr<FFragmentEntry> Entry) const;
			FText GetTagTypeText(TSharedPtr<FFragmentEntry> Entry) const;
			FText GetTagPresenceText(TSharedPtr<FFragmentEntry> Entry) const;

			TWeakPtr<SQueryEditor> OwnerRequirementsEditor;
			bool bShowTags;
		};

		TArray<TSharedPtr<FName>> FragmentAccessNames;
		TArray<TSharedPtr<FName>> FragmentPresenceNames;
		TMap<FName, EMassFragmentAccess> FragmentAccessMap;
		TMap<FName, EMassFragmentPresence> FragmentPresenceMap;

		TSharedPtr<SListView<TSharedPtr<FFragmentEntry>>> FragmentListView;
		TSharedPtr<SListView<TSharedPtr<FFragmentEntry>>> TagListView;
	};

} // namespace UE::MassDebugger
