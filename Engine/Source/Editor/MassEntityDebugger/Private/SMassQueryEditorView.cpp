// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMassQueryEditorView.h"
#include "MassDebuggerModel.h"
#include "Dom/JsonObject.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/FileHelper.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SMassQueryEditor"

namespace UE::MassDebugger
{
	void SQueryEditorView::Construct(const FArguments& InArgs, TSharedRef<FMassDebuggerModel> InDebuggerModel)
	{
		DebuggerModel = InDebuggerModel;
		InDebuggerModel->OnRefreshDelegate.AddSP(this, &SQueryEditorView::OnRefresh);
		InDebuggerModel->OnQueriesChangedDelegate.AddSP(this, &SQueryEditorView::OnRefresh);

		ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot().AutoWidth().Padding(2)
					[
						SNew(SButton)
						.OnClicked(this, &SQueryEditorView::OnSaveQuery)
						.ToolTipText(LOCTEXT("SaveQuery_Tooltip", "Save selected query"))
						.Content()
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.Save"))
						]
					]
					+ SHorizontalBox::Slot().AutoWidth().Padding(2)
					[
						SNew(SButton)
						.OnClicked(this, &SQueryEditorView::OnSaveAllQueries)
						.ToolTipText(LOCTEXT("SaveAllQueries_Tooltip", "Save all queries"))
						.Content()
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush("Icons.SaveModified"))
						]
					]
					+ SHorizontalBox::Slot().AutoWidth().Padding(2)
					[
						SNew(SButton)
						.OnClicked(this, &SQueryEditorView::OnCopyQuery)
						.ToolTipText(LOCTEXT("CopyQuery_Tooltip", "Copy selected query as JSON"))
						.Content()
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("GenericCommands.Copy"))
						]
					]
					+ SHorizontalBox::Slot().AutoWidth().Padding(2)
					[
						SNew(SButton)
						.OnClicked(this, &SQueryEditorView::OnPasteQuery)
						.ToolTipText(LOCTEXT("PasteQuery_Tooltip", "Paste JSON from clipboard as new query"))
						.Content()
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush("GenericCommands.Paste"))
						]
					]
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2)
				[
					SNew(SButton)
					.Text(LOCTEXT("AddQuery", "Add Query"))
					.OnClicked(this, &SQueryEditorView::OnAddQuery)
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2)
				[
					SNew(SSeparator)
				]
				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				.Padding(2)
				[
					SAssignNew(QueryListView, SListView<TSharedPtr<FEditableQuery>>)
					.ListItemsSource(&InDebuggerModel->QueryList)
					.SelectionMode(ESelectionMode::Single)
					.OnGenerateRow(this, &SQueryEditorView::OnGenerateQueryRow)
					.OnSelectionChanged_Lambda([this, WeakModel = InDebuggerModel.ToWeakPtr()](TSharedPtr<FEditableQuery> InQuery, ESelectInfo::Type InSelectType)
					{
						if(TSharedPtr<FMassDebuggerModel> Model = WeakModel.Pin())
						{
							if (!InQuery.IsValid())
							{
								if (SelectedQuery.IsValid() && Model->QueryList.Contains(SelectedQuery))
								{
									QueryListView->SetSelection(SelectedQuery);
								}
								else
								{
									SelectedQuery = Model->QueryList.Num() > 0 ? Model->QueryList[0] : InQuery;
									QueryEditor->SetQuery(SelectedQuery);
								}
							}
							else
							{
								SelectedQuery = InQuery;
								QueryEditor->SetQuery(SelectedQuery);
							}
						}
					})
				]
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(4)
			[
				SAssignNew(QueryEditor, SQueryEditor)
				.DebuggerModel(InDebuggerModel)
				.InitialRequirements(FMassFragmentRequirements())
				.OnRequirementsChanged(this, &SQueryEditorView::OnRequirementsChanged)
			]
		];

		InDebuggerModel->RegisterQueryEditor(SharedThis(this));
	}

	void SQueryEditorView::OnRefresh()
	{
		if (QueryListView.IsValid())
		{
			QueryListView->RequestListRefresh();
		}
	}

	TSharedRef<ITableRow> SQueryEditorView::OnGenerateQueryRow(TSharedPtr<FEditableQuery> InItem, const TSharedRef<STableViewBase>& OwnerTable)
	{
		return SNew(STableRow<TSharedPtr<FEditableQuery>>, OwnerTable)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(0.3f)
			.VAlign(VAlign_Center)
			.Padding(2)
			[
				SNew(STextBlock)
				.Text_Lambda([InItem]()
				{
					return InItem.IsValid()
						? FText::FromString(InItem->Name)
						: FText::GetEmpty();
				})
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2)
			[
				SNew(SButton)
				.Text(LOCTEXT("ViewEntities", "View Entities"))
				.OnClicked(this, &SQueryEditorView::OnViewEntities, InItem)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2)
			[
				SNew(SButton)
				.OnClicked(this, &SQueryEditorView::OnDeleteQuery, InItem)
				[
					SNew(SImage)
					.Image(FCoreStyle::Get().GetBrush("Icons.Delete"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		];
	}

	FReply SQueryEditorView::OnAddQuery()
	{
		TSharedPtr<FEditableQuery> NewQuery = MakeShared<FEditableQuery>();
		NewQuery->Name = TEXT("NewQuery");

		if (TSharedPtr<FMassDebuggerModel> Model = DebuggerModel.Pin())
		{
			Model->QueryList.Add(NewQuery);
			if (QueryListView.IsValid())
			{
				QueryListView->RequestListRefresh();
				QueryListView->SetSelection(NewQuery);
				QueryListView->RequestScrollIntoView(NewQuery);
			}
			Model->RefreshQueries();
		}
		return FReply::Handled();
	}

	FReply SQueryEditorView::OnDeleteQuery(TSharedPtr<FEditableQuery> InItem)
	{
		if (TSharedPtr<FMassDebuggerModel> Model = DebuggerModel.Pin())
		{
			Model->QueryList.Remove(InItem);
			if (QueryListView.IsValid())
			{
				QueryListView->RequestListRefresh();
			}

			if (SelectedQuery == InItem && QueryEditor.IsValid())
			{
				SelectedQuery.Reset();
				QueryEditor->SetQuery(SelectedQuery);
			}
			Model->RefreshQueries();
		}
		return FReply::Handled();
	}

	FReply SQueryEditorView::OnEditQuery(TSharedPtr<FEditableQuery> InItem)
	{
		if (!InItem.IsValid() || !QueryEditor.IsValid())
		{
			return FReply::Unhandled();
		}

		SelectedQuery = InItem;

		QueryEditor->SetQuery(SelectedQuery);
		return FReply::Handled();
	}

	FReply SQueryEditorView::OnViewEntities(TSharedPtr<FEditableQuery> InItem)
	{
		TSharedPtr<FMassDebuggerModel> Model = DebuggerModel.Pin();
		if (InItem.IsValid() && Model.IsValid() && !Model->IsStale())
		{
			FMassEntityQuery Query = InItem->BuildEntityQuery(Model->Environment->GetMutableEntityManager().ToSharedRef());
			Model->ShowEntitiesView(0, Query);
		}
		return FReply::Handled();
	}

	void SQueryEditorView::OnRequirementsChanged(TSharedPtr<FEditableQuery>& Query)
	{
	}

	void SQueryEditorView::ShowQuery(const FMassEntityQuery& InQuery, const FString& InQueryName)
	{
		if (TSharedPtr<FMassDebuggerModel> Model = DebuggerModel.Pin())
		{
			TSharedPtr<FEditableQuery> NewQuery = MakeShared<FEditableQuery>();
			NewQuery->Name = InQueryName;
			NewQuery->InitFromEntityQuery(InQuery, *Model);
			Model->QueryList.Add(NewQuery);
			if (QueryListView.IsValid())
			{
				QueryListView->RequestListRefresh();
				QueryListView->SetSelection(NewQuery);
				QueryListView->RequestScrollIntoView(NewQuery);
			}
			Model->RefreshQueries();
		}
	}

	FReply SQueryEditorView::OnSaveQuery()
	{
		if (!SelectedQuery.IsValid())
		{
			return FReply::Handled();
		}

		const FString FileName = TEXT("Queries.json");
		const FString SavedDir = FPaths::ProjectSavedDir() / TEXT("MassDebugger");
		IFileManager::Get().MakeDirectory(*SavedDir, true);
		const FString FullPath = SavedDir / FileName;

		TSharedPtr<FJsonObject> RootObject = MakeShared<FJsonObject>();
		TArray<TSharedPtr<FJsonValue>> QueriesArray;
		if (FPaths::FileExists(FullPath))
		{
			FString Existing;
			FFileHelper::LoadFileToString(Existing, *FullPath);
			TSharedPtr<FJsonObject> Loaded;
			const TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(Existing);
			if (FJsonSerializer::Deserialize(Reader, Loaded) && Loaded.IsValid())
			{
				const TArray<TSharedPtr<FJsonValue>>* QueriesArrayJson;;
				Loaded->TryGetArrayField(TEXT("Queries"), QueriesArrayJson);
			}
		}

		const TSharedPtr<FJsonObject> SelJson = SelectedQuery->SerializeToJson();
		bool bReplaced = false;
		for (int32 i = 0; i < QueriesArray.Num(); ++i)
		{
			if (QueriesArray[i]->AsObject()->GetStringField(TEXT("Name")) == SelectedQuery->Name)
			{
				QueriesArray[i] = MakeShared<FJsonValueObject>(SelJson);
				bReplaced = true;
				break;
			}
		}
		if (!bReplaced)
		{
			QueriesArray.Add(MakeShared<FJsonValueObject>(SelJson));
		}

		RootObject->SetArrayField(TEXT("Queries"), QueriesArray);

		FString OutString;
		const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutString);
		FJsonSerializer::Serialize(RootObject.ToSharedRef(), Writer);
		FFileHelper::SaveStringToFile(OutString, *FullPath);

		return FReply::Handled();
	}

	FReply SQueryEditorView::OnSaveAllQueries()
	{
		if (TSharedPtr<FMassDebuggerModel> Model = DebuggerModel.Pin())
		{
			const FString FileName = TEXT("Queries.json");
			const FString SavedDir = FPaths::ProjectSavedDir() / TEXT("MassDebugger");
			IFileManager::Get().MakeDirectory(*SavedDir, /*Tree=*/true);
			const FString FullPath = SavedDir / FileName;

			TSharedPtr<FJsonObject> RootObject = MakeShared<FJsonObject>();
			TArray<TSharedPtr<FJsonValue>> QueriesArray;
			for (TSharedPtr<FEditableQuery>& Query : Model->QueryList)
			{
				QueriesArray.Add(MakeShared<FJsonValueObject>(Query->SerializeToJson()));
			}
			RootObject->SetArrayField(TEXT("Queries"), MoveTemp(QueriesArray));

			FString OutString;
			const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutString);
			FJsonSerializer::Serialize(RootObject.ToSharedRef(), Writer);
			FFileHelper::SaveStringToFile(OutString, *FullPath);
		}
		return FReply::Handled();
	}

	FReply SQueryEditorView::OnCopyQuery()
	{
		if (!SelectedQuery.IsValid())
		{
			return FReply::Handled();
		}

		FString JsonString;
		const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&JsonString);
		FJsonSerializer::Serialize(SelectedQuery->SerializeToJson().ToSharedRef(), Writer);

		FPlatformApplicationMisc::ClipboardCopy(*JsonString);
		return FReply::Handled();
	}

	FReply SQueryEditorView::OnPasteQuery()
	{
		TSharedPtr<FMassDebuggerModel> Model = DebuggerModel.Pin();

		FString ClipboardText;
		FPlatformApplicationMisc::ClipboardPaste(ClipboardText);

		TSharedPtr<FEditableQuery> NewQuery = MakeShared<FEditableQuery>();
		if (Model && NewQuery->DeserializeFromJsonString(ClipboardText))
		{
			Model->QueryList.Add(NewQuery);
			if (QueryListView.IsValid())
			{
				QueryListView->RequestListRefresh();
				QueryListView->SetSelection(NewQuery);
				QueryListView->RequestScrollIntoView(NewQuery);
			}
			Model->RefreshQueries();
		}
		return FReply::Handled();
	}

	void SQueryEditorView::OnQueriesChanged()
	{
		TSharedPtr<FMassDebuggerModel> Model = DebuggerModel.Pin();
		if (!Model)
		{
			return;
		}

		if (QueryListView.IsValid())
		{
			QueryListView->RequestListRefresh();
		}

		if (Model->QueryList.Num() > 0)
		{
			SelectedQuery = Model->QueryList[0];
			QueryListView->SetSelection(SelectedQuery);
		}
		else
		{
			SelectedQuery.Reset();
		}
		if (QueryEditor.IsValid())
		{
			QueryEditor->SetQuery(SelectedQuery);
		}
	}
} // namespace UE::MassDebugger

#undef LOCTEXT_NAMESPACE
