// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "SMassDebuggerViewBase.h"
#include "MassEntityQuery.h"
#include "SMassQueryEditor.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class FJsonObject;

namespace UE::MassDebugger
{
	class SQueryEditorView : public SMassDebuggerViewBase
	{
	public:
		SLATE_BEGIN_ARGS(SQueryEditorView) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedRef<FMassDebuggerModel> InDebuggerModel);

		// ~SMassDebuggerViewBase interface
		virtual void OnRefresh() override;
		virtual void OnProcessorsSelected(TConstArrayView<TSharedPtr<FMassDebuggerProcessorData>> SelectedProcessors, ESelectInfo::Type SelectInfo) override
		{}
		virtual void OnArchetypesSelected(TConstArrayView<TSharedPtr<FMassDebuggerArchetypeData>> SelectedArchetypes, ESelectInfo::Type SelectInfo) override
		{}

		void ShowQuery(const FMassEntityQuery& InQuery, const FString& InQueryName);

		FReply OnSaveQuery();
		FReply OnSaveAllQueries();
		FReply OnCopyQuery();
		FReply OnPasteQuery();

	protected:

		TSharedPtr<SListView<TSharedPtr<FEditableQuery>>> QueryListView;
		TSharedPtr<SQueryEditor> QueryEditor;
		TSharedPtr<FEditableQuery> SelectedQuery;

		TWeakPtr<FMassDebuggerModel> DebuggerModel;

		TSharedRef<ITableRow> OnGenerateQueryRow(TSharedPtr<FEditableQuery> InItem, const TSharedRef<STableViewBase>& OwnerTable);
		FReply OnAddQuery();
		FReply OnDeleteQuery(TSharedPtr<FEditableQuery> InItem);
		FReply OnEditQuery(TSharedPtr<FEditableQuery> InItem);
		FReply OnViewEntities(TSharedPtr<FEditableQuery> InItem);
		void OnRequirementsChanged(TSharedPtr<FEditableQuery>& Query);
		void OnQueriesChanged();
	};

} // namespace UE::MassDebugger
