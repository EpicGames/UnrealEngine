// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassSettings.h"
#include "MassDebuggerSettings.generated.h"


#define GET_MASSDEBUGGER_CONFIG_VALUE(a) \
	UE_DEPRECATED_MACRO(5.8, "Use GetDefault<UMassDebuggerSettings>() (or GetMutableDefault) directly.") \
	(GetMutableDefault<UMassDebuggerSettings>()->a)

UCLASS(MinimalAPI, config = EditorPerProjectUserSettings, DisplayName = "Mass Debugger")
class UMassDebuggerSettings : public UMassModuleSettings
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, config, Category=Debugger)
	bool bStripMassPrefix = true;
};
